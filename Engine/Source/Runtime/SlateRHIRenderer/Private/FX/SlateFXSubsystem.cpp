// Copyright Epic Games, Inc. All Rights Reserved.

#include "FX/SlateFXSubsystem.h"

#include "Engine/Engine.h"
#include "FX/SlatePostBufferBlur.h"
#include "FX/SlateRHIPostBufferBlur.h"
#include "FX/SlateRHIPostBufferProcessor.h"
#include "SlateRHIRendererSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateFXSubsystem)

TSharedPtr<FSlateRHIPostBufferProcessorProxy> USlateFXSubsystem::GetPostProcessorProxy(ESlatePostRT InSlatePostBufferBit)
{
	TSharedPtr<FSlateRHIPostBufferProcessorProxy> Result = nullptr;

	if (GEngine)
	{
		if (USlateFXSubsystem* SlateFXSubsystem = GEngine->GetEngineSubsystem<USlateFXSubsystem>())
		{
			Result = SlateFXSubsystem->GetSlatePostProcessorProxy(InSlatePostBufferBit);
		}
	}

	return Result;
}

void USlateFXSubsystem::BeginDestroy()
{
	// Flush rendering commands since this subsystem can be used in render thread
	FlushRenderingCommands();

	Super::BeginDestroy();
}

TSharedPtr<FSlateRHIPostBufferProcessorProxy> USlateFXSubsystem::GetSlatePostProcessorProxy(ESlatePostRT InPostBufferBit)
{
	if (TSharedPtr<FSlateRHIPostBufferProcessorProxy>* ProcessorProxy = SlatePostBufferProcessorProxies.Find(InPostBufferBit))
	{
		return *ProcessorProxy;
	}

	return nullptr;
}

USlatePostBufferProcessor* USlateFXSubsystem::GetSlatePostProcessor(ESlatePostRT InPostBufferBit)
{
	return Super::GetSlatePostProcessor(InPostBufferBit);
}

TSharedPtr<FSlateRHIPostBufferProcessorProxy> USlateFXSubsystem::CreatePostBufferProcessorProxy(TSubclassOf<USlatePostBufferProcessor> PostProcessorClass)
{
	if (PostProcessorClass)
	{
		if (PostProcessorClass->IsChildOf(USlatePostBufferBlur::StaticClass()))
		{
			return MakeShared<FSlatePostBufferBlurProxy>();
		}
		//Add other processor proxy here.

		UE_LOGF(LogSlate, Warning, "Failed to create FSlateRHIPostBufferProcessorProxy for USlatePostBufferProcessor %ls", *(PostProcessorClass.Get()->GetName()));
	}
	else
	{
		UE_LOGF(LogSlate, Warning, "Failed to create FSlateRHIPostBufferProcessorProxy for an invalid USlatePostBufferProcessor");
	}

	return TSharedPtr<FSlateRHIPostBufferProcessorProxy>();
}

void USlateFXSubsystem::OnInitProcessors()
{
	SlatePostBufferProcessorProxies.Empty();

	if (const USlateRHIRendererSettings* SlateRendererSettings = USlateRHIRendererSettings::Get())
	{
		for (ESlatePostRT SlatePostBufferBit : TEnumRange<ESlatePostRT>())
		{
			const FSlatePostSettings& PostSetting = SlateRendererSettings->GetSlatePostSetting(SlatePostBufferBit);
			if (PostSetting.bEnabled && PostSetting.PostProcessorClass)
			{
				if (TObjectPtr<USlatePostBufferProcessor> BufferProcessor = NewObject<USlatePostBufferProcessor>(this, PostSetting.PostProcessorClass))
				{
					TSharedPtr<FSlateRHIPostBufferProcessorProxy> PostProcessorProxy = CreatePostBufferProcessorProxy(PostSetting.PostProcessorClass);
					if (ensure(PostProcessorProxy))
					{
						BufferProcessor->SetRenderThreadProxy(PostProcessorProxy);
						PostProcessorProxy->SetOwningProcessorObject(BufferProcessor);
						SlatePostBufferProcessors.Add(SlatePostBufferBit, BufferProcessor);
						SlatePostBufferProcessorProxies.Add(SlatePostBufferBit, PostProcessorProxy);
						PostProcessorProxy->OnUpdateValuesRenderThread();
					}
				}
			}
		}
	}
}

void USlateFXSubsystem::OnCleanupProcessors()
{
	SlatePostBufferProcessorProxies.Empty();
}
