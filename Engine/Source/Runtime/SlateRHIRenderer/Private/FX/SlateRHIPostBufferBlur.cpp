// Copyright Epic Games, Inc. All Rights Reserved.

#include "FX/SlateRHIPostBufferBlur.h"
#include "FX/SlatePostBufferBlur.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "RHIResources.h"
#include "RHIUtilities.h"

/////////////////////////////////////////////////////
// FSlatePostBufferBlurProxy

void FSlatePostBufferBlurProxy::PostProcess_Renderthread(FRDGBuilder& GraphBuilder, const FScreenPassTexture& InputTexture, const FScreenPassTexture& OutputTexture)
{
	RDG_EVENT_SCOPE(GraphBuilder, "SlatePostBufferBlur");

	if (GaussianBlurStrength_RenderThread < UE_SMALL_NUMBER)
	{
		AddSlatePostProcessCopy(GraphBuilder, InputTexture, OutputTexture);
	}
	else
	{
		FSlatePostProcessSimpleBlurPassInputs BlurInputs;
		BlurInputs.InputTexture = InputTexture;
		BlurInputs.OutputTexture = OutputTexture;
		BlurInputs.Strength = GaussianBlurStrength_RenderThread;

		AddSlatePostProcessBlurPass(GraphBuilder, BlurInputs);
	}
}

void FSlatePostBufferBlurProxy::OnUpdateValuesRenderThread()
{
	// Don't issue multiple updates in a single frame from the CPU based on dirty values
	if (!ParamUpdateFence.IsFenceComplete())
	{
		return;
	}

	// Only issue an update when parent exists & values are different
	if (USlatePostBufferBlur* ParentBlurObject = Cast<USlatePostBufferBlur>(ParentObject))
	{
		if (ParentBlurObject->GaussianBlurStrength != GaussianBlurStrength_RenderThread)
		{
			// Blur strengths can be updated from renderthread during draw or gamethread,
			// if our parent object value matches the predraw then don't update the renderthread value.
			// Instead we need to update our parent object's value to match the last value from renderthread
			bool bUpdatedInRenderThread = ParentBlurObject->GaussianBlurStrength == GaussianBlurStrengthPreDraw;

			if (bUpdatedInRenderThread)
			{
				ParentBlurObject->GaussianBlurStrength = GaussianBlurStrength_RenderThread;
				GaussianBlurStrengthPreDraw = GaussianBlurStrength_RenderThread;
			}
			else
			{
				// Explicit param copy to avoid renderthread from reading value during gamethread write
				float GaussianBlurStrengthCopy = ParentBlurObject->GaussianBlurStrength;
				GaussianBlurStrengthPreDraw = GaussianBlurStrengthCopy;

				// Execute param copy in a render command to safely update value on renderthread without race conditions
				TWeakPtr<FSlatePostBufferBlurProxy> TempWeakThis = SharedThis(this);
				ENQUEUE_RENDER_COMMAND(FUpdateValuesRenderThreadFX_Blur)([TempWeakThis, GaussianBlurStrengthCopy](FRHICommandListImmediate& RHICmdList)
				{
					if (TSharedPtr<FSlatePostBufferBlurProxy> SharedThisPin = TempWeakThis.Pin())
					{
						SharedThisPin->GaussianBlurStrength_RenderThread = GaussianBlurStrengthCopy;
					}
				});

				// Issue fence to prevent multiple updates in a single frame
				ParamUpdateFence.BeginFence();
			}
		}
	}
}