// Copyright Epic Games, Inc. All Rights Reserved.

#include "Slate3DRenderer.h"
#include "Fonts/FontCache.h"
#include "Materials/MaterialRenderProxy.h"
#include "Widgets/SWindow.h"
#include "SceneUtils.h"
#include "SlateRHIRenderer.h"
#include "Rendering/ElementBatcher.h"
#include "Types/SlateVector2.h"
#include "RenderGraphUtils.h"
#include "SlatePostProcessor.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"

DECLARE_GPU_STAT_NAMED(Slate3D, TEXT("Slate 3D"));

FSlate3DRenderer::FSlate3DRenderer( TSharedRef<FSlateFontServices> InSlateFontServices, TSharedRef<FSlateRHIResourceManager> InResourceManager, bool bInUseGammaCorrection )
	: SlateFontServices( InSlateFontServices )
	, ResourceManager( InResourceManager )
	, bGammaCorrection(bInUseGammaCorrection)
{
	RenderTargetPolicy = MakeShareable( new FSlateRHIRenderingPolicy( SlateFontServices, ResourceManager ) );

	ElementBatcher = MakeUnique<FSlateElementBatcher>(RenderTargetPolicy.ToSharedRef());
}

void FSlate3DRenderer::Cleanup()
{
	BeginCleanup(this);
}

void FSlate3DRenderer::SetUseGammaCorrection(bool bInUseGammaCorrection)
{
	bGammaCorrection = bInUseGammaCorrection;
}

void FSlate3DRenderer::SetApplyColorDeficiencyCorrection(bool bInAllowColorDeficiencyCorrection)
{
	bAllowColorDeficiencyCorrection = bInAllowColorDeficiencyCorrection;
}

FSlateDrawBuffer& FSlate3DRenderer::AcquireDrawBuffer()
{
	FreeBufferIndex = (FreeBufferIndex + 1) % NUM_DRAW_BUFFERS;
	FSlateDrawBuffer* Buffer = &DrawBuffers[FreeBufferIndex];

	while (!Buffer->Lock())
	{
		FlushRenderingCommands();

		UE_LOGF(LogSlate, Log, "Slate: Had to block on waiting for a draw buffer");
		FreeBufferIndex = (FreeBufferIndex + 1) % NumDrawBuffers;

		Buffer = &DrawBuffers[FreeBufferIndex];
	}

	Buffer->ClearBuffer();

	return *Buffer;
}

void FSlate3DRenderer::ReleaseDrawBuffer(FSlateDrawBuffer& InWindowDrawBuffer)
{
#if DO_CHECK
	bool bFound = false;
	for (int32 Index = 0; Index < NUM_DRAW_BUFFERS; ++Index)
	{
		if (&DrawBuffers[Index] == &InWindowDrawBuffer)
		{
			bFound = true;
			break;
		}
	}
	ensureMsgf(bFound, TEXT("It release a DrawBuffer that is not a member of the Slate3DRenderer"));
#endif

	ENQUEUE_RENDER_COMMAND(SlateReleaseDrawBufferCommand)(
		[DrawBuffer = &InWindowDrawBuffer](FRHICommandListImmediate& RHICmdList)
	{
		DrawBuffer->Unlock(FRDGBuilder::GetAsyncExecuteTask());
	});
}

void FSlate3DRenderer::DrawWindow_GameThread(FSlateDrawBuffer& DrawBuffer)
{
	check( IsInGameThread() );

	const TSharedRef<FSlateFontCache> FontCache = SlateFontServices->GetGameThreadFontCache();

	const TArray<TSharedRef<FSlateWindowElementList>>& WindowElementLists = DrawBuffer.GetWindowElementLists();

	for (int32 WindowIndex = 0; WindowIndex < WindowElementLists.Num(); WindowIndex++)
	{
		FSlateWindowElementList& ElementList = *WindowElementLists[WindowIndex];

		SWindow* Window = ElementList.GetPaintWindow();

		if (Window)
		{
			const FVector2D WindowSize = Window->GetSizeInScreen();
			if (WindowSize.X > 0.0 && WindowSize.Y > 0.0)
			{
				// Add all elements for this window to the element batcher
				ElementBatcher->AddElements(ElementList);

				// Update the font cache with new text after elements are batched
				FontCache->UpdateCache();

				// All elements for this window have been batched and rendering data updated
				ElementBatcher->ResetBatches();
			}
		}
	}
}

void FSlate3DRenderer::DrawWindowToTarget_RenderThread(FRDGBuilder& GraphBuilder, const FRenderThreadUpdateContext& Context)
{
	QUICK_SCOPE_CYCLE_COUNTER(Stat_Slate_WidgetRendererRenderThread);
	RDG_EVENT_SCOPE_STAT(GraphBuilder, Slate3D, "SlateRenderToTarget");

	check(Context.RenderTarget);

	const TArray<TSharedRef<FSlateWindowElementList>>& WindowsToDraw = Context.WindowDrawBuffer->GetWindowElementLists();

	FRDGTexture* SlateElementsTexture = RegisterExternalTexture(GraphBuilder, Context.RenderTarget->GetRenderTargetTexture(), TEXT("SlateElementsTexture"));
	const FIntPoint SlateElementsExtent = SlateElementsTexture->Desc.Extent;

	FRDGTexture* SlateStencilTexture = nullptr;
	bool bStencilClippingRequired = false;

	TArray<FSlateElementsBuffers, FRDGArrayAllocator> SlateElementsBuffers;
	SlateElementsBuffers.Reserve(WindowsToDraw.Num());

	for (const TSharedRef<FSlateWindowElementList>& WindowElementList : WindowsToDraw)
	{
		FSlateBatchData& BatchData = WindowElementList->GetBatchData();

		SlateElementsBuffers.Emplace(BuildSlateElementsBuffers(GraphBuilder, BatchData));

		bStencilClippingRequired |= BatchData.IsStencilClippingRequired();
	}

	if (bStencilClippingRequired)
	{
		SlateStencilTexture = GraphBuilder.CreateTexture(
			FRDGTextureDesc::Create2D(SlateElementsExtent, PF_DepthStencil, FClearValueBinding::DepthZero, GetSlateTransientDepthStencilFlags()),
			TEXT("SlateStencilTexture"));
	}

	ERenderTargetLoadAction ElementsLoadAction = Context.bClearTarget ? ERenderTargetLoadAction::EClear : ERenderTargetLoadAction::ELoad;
	
	const auto ConsumeLoadAction = [] (ERenderTargetLoadAction& InOutLoadAction)
	{
		ERenderTargetLoadAction LoadAction = InOutLoadAction;
		InOutLoadAction = ERenderTargetLoadAction::ELoad;
		return LoadAction;
	};

	for (int32 WindowElementIndex = 0; WindowElementIndex < WindowsToDraw.Num(); ++WindowElementIndex)
	{
		const TSharedRef<FSlateWindowElementList>& WindowElementList = WindowsToDraw[WindowElementIndex];

		FSlateBatchData& BatchData = WindowElementList->GetBatchData();

		if (BatchData.GetRenderBatches().IsEmpty())
		{
			continue;
		}

		const FVector ElementsOffset(Context.WindowDrawBuffer->ViewOffset, 0.0f);
		const FMatrix ElementsMatrix(FTranslationMatrix::Make(ElementsOffset) * CreateSlateProjectionMatrix(SlateElementsExtent.X, SlateElementsExtent.Y));

		FSlateDrawElementsPassInputs DrawElementsInputs =
		{
			  .StencilTexture        = SlateStencilTexture
			, .ElementsTexture       = SlateElementsTexture
			, .ElementsLoadAction    = ConsumeLoadAction(ElementsLoadAction)
			, .ElementsBuffers       = SlateElementsBuffers[WindowElementIndex]
			, .ElementsMatrix        = FMatrix44f(ElementsMatrix)
			, .ElementsOffset        = FVector2f(ElementsOffset.X, ElementsOffset.Y)
			, .Time                  = FGameTime::CreateDilated(Context.RealTimeSeconds, Context.DeltaRealTimeSeconds, Context.WorldTimeSeconds, Context.DeltaTimeSeconds)
			, .bAllowGammaCorrection = bGammaCorrection
		};

		AddSlateDrawElementsPass(GraphBuilder, *RenderTargetPolicy, DrawElementsInputs, BatchData.GetRenderBatches(), BatchData.GetFirstRenderBatchIndex());
	}

	if (ConsumeLoadAction(ElementsLoadAction) == ERenderTargetLoadAction::EClear)
	{
		AddClearRenderTargetPass(GraphBuilder, SlateElementsTexture);
	}
}