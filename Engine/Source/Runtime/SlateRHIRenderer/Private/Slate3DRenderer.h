// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Rendering/SlateRenderer.h"
#include "Interfaces/ISlate3DRenderer.h"
#include "SlateRHIResourceManager.h"
#include "Rendering/SlateDrawBuffer.h"
#include "RenderDeferredCleanup.h"
#include "SlateRHIRenderingPolicy.h"

class FSlateElementBatcher;
class FTextureRenderTarget2DResource;

class FSlate3DRenderer : public ISlate3DRenderer, public FDeferredCleanupInterface
{
public:
	FSlate3DRenderer( TSharedRef<FSlateFontServices> InSlateFontServices, TSharedRef<FSlateRHIResourceManager> InResourceManager, bool bUseGammaCorrection = false );

	void Cleanup();

	virtual void SetUseGammaCorrection(bool bUseGammaCorrection) override;
	virtual void SetApplyColorDeficiencyCorrection(bool bApplyColorCorrection) override;

	virtual FSlateDrawBuffer& AcquireDrawBuffer() override;
	virtual void ReleaseDrawBuffer(FSlateDrawBuffer& DrawBuffer) override;
	virtual void DrawWindow_GameThread(FSlateDrawBuffer& DrawBuffer) override;
	virtual void DrawWindowToTarget_RenderThread(FRDGBuilder& GraphBuilder, const struct FRenderThreadUpdateContext& Context) override;

private:

	/** Double buffered draw buffers so that the rendering thread can be rendering windows while the game thread is setting up for next frame */
	static const int32 NUM_DRAW_BUFFERS = 4;
	FSlateDrawBuffer DrawBuffers[NUM_DRAW_BUFFERS];

	/** The font services to use for rendering text */
	TSharedRef<FSlateFontServices> SlateFontServices;

	/** Texture manager for accessing textures on the game thread */
	TSharedRef<FSlateRHIResourceManager> ResourceManager;

	/** The rendering policy to use for drawing to the render target */
	TSharedPtr<class FSlateRHIRenderingPolicy> RenderTargetPolicy;

	/** Element batcher that renders draw elements */
	TUniquePtr<FSlateElementBatcher> ElementBatcher;

	/** The draw buffer that is currently free for use by the game thread */
	uint8 FreeBufferIndex;

	/** Set to true when the render target was cleared and prevent another clear to be called for nothing.**/
	bool bRenderTargetWasCleared = false;

	bool bAllowColorDeficiencyCorrection = true;
	bool bGammaCorrection;
};
