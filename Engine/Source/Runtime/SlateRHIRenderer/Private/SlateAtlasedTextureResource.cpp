// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateAtlasedTextureResource.h"
#include "Slate/SlateTextureAtlasInterface.h"

FSlateAtlasedTextureResource::FSlateAtlasedTextureResource(UTexture* InTexture)
	: FSlateBaseUTextureResource(InTexture)
{
}

FSlateAtlasedTextureResource::~FSlateAtlasedTextureResource()
{
	for ( FObjectResourceMap::TIterator ProxyIt(ProxyMap); ProxyIt; ++ProxyIt )
	{
		FSlateShaderResourceProxy* Proxy = ProxyIt.Value();
		delete Proxy;
	}
}

bool FSlateAtlasedTextureResource::IsResourceValid() const
{
	return TextureObject && TextureObject->IsValidLowLevel();
}

FSlateShaderResourceProxy* FSlateAtlasedTextureResource::FindOrCreateAtlasedProxy(UObject* InAtlasedObject, const FSlateAtlasData& AtlasData)
{
	FSlateShaderResourceProxy* Proxy = ProxyMap.FindRef(InAtlasedObject);
	if ( Proxy == nullptr )
	{
		// when we use image-DrawAsBox with PaperSprite, we need to change its actual size as its actual dimension.
		FVector2D ActualSize(TextureObject->GetSurfaceWidth() * AtlasData.SizeUV.X, TextureObject->GetSurfaceHeight() * AtlasData.SizeUV.Y);

		Proxy = new FSlateShaderResourceProxy();
		Proxy->Resource = this;
		Proxy->ActualSize = ActualSize.IntPoint();
		Proxy->StartUV = FVector2f(AtlasData.StartUV);	// LWC_TODO: Precision loss
		Proxy->SizeUV = FVector2f(AtlasData.SizeUV);	// LWC_TODO: Precision loss

		ProxyMap.Add(InAtlasedObject, Proxy);
	}

	return Proxy;
}
