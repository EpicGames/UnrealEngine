// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateUTextureResource.h"

struct FSlateAtlasData;

/**
 * A resource for rendering a sub-rect of a UTexture atlas object in Slate. Each 
 * ISlateTextureAtlasInterface* should uniquely identify a subregion of the UTexture atlas.
 */
class FSlateAtlasedTextureResource : public FSlateBaseUTextureResource
{
public:
	/** Initializes a new atlased UTexture resource, the incoming texture should be the entire atlas. */
	FSlateAtlasedTextureResource(UTexture* InTexture);
	
	/** Destructor */
	virtual ~FSlateAtlasedTextureResource();

	// Begin FSlateShaderResource Interface
	virtual bool IsResourceValid() const override;
	// End FSlateShaderResource Interface
	
	/**
	 * Finds or creates the rendering proxy for a given atlas'ed object.
	 * @param InAtlasedObject The atlased object to find a rendering proxy for.
	 * @param InAtlasedObjectInterface The Interface pointer for the InAtlasedObject.
	 */
	FSlateShaderResourceProxy* FindOrCreateAtlasedProxy(UObject* InAtlasedObject, const FSlateAtlasData& AtlasData);
	
public:
	typedef TMap<TWeakObjectPtr<UObject>, FSlateShaderResourceProxy* > FObjectResourceMap;

	/** Map of all the atlased resources. */
	FObjectResourceMap ProxyMap;
};
