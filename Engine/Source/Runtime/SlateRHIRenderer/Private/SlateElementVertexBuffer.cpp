// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateElementVertexBuffer.h"

DEFINE_STAT(STAT_SlateVertexBufferMemory);

TGlobalResource<FSlateStencilClipVertexBuffer> GSlateStencilClipVertexBuffer;