// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "Rendering/RenderingCommon.h"
#include "SlateGlobals.h"
#include "RHI.h"
#include "RHICommandList.h"
#include "RHIResourceUtils.h"
#include "RenderResource.h"
#include "Containers/ResourceArray.h"

DECLARE_MEMORY_STAT_EXTERN(TEXT("Vertex Buffer Memory (GPU)"), STAT_SlateVertexBufferMemory, STATGROUP_SlateMemory, );

/**
 * Vertex buffer containing all Slate vertices
 */
template <typename VertexType>
class TSlateElementVertexBuffer : public FVertexBuffer
{
public:
	TSlateElementVertexBuffer()
		: BufferSize(0)
		, MinBufferSize(0)
		, BufferUsageSize(0)
	{}

	~TSlateElementVertexBuffer() = default;

	void Init( int32 MinNumVertices )
	{
		MinBufferSize = sizeof(VertexType) * FMath::Max( MinNumVertices, 100 );

		if ( IsInRenderingThread() )
		{
			InitResource(FRHICommandListImmediate::Get());
		}
		else
		{
			BeginInitResource(this);
		}
	}

	void Destroy()
	{
		if ( IsInRenderingThread() )
		{
			ReleaseResource();
		}
		else
		{
			BeginReleaseResource(this);
		}
	}

	/** Initializes the vertex buffers RHI resource. */
	virtual void InitRHI(FRHICommandListBase& RHICmdList)
	{
		if( !IsValidRef(VertexBufferRHI) )
		{
			check( MinBufferSize > 0 )
	
			SetBufferSize(MinBufferSize);

			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateVertex(TEXT("SlateElementVertices"), MinBufferSize)
				.AddUsage(EBufferUsageFlags::Dynamic)
				.DetermineInitialState();

			VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);

			// Ensure the vertex buffer could be created
			check(IsValidRef(VertexBufferRHI));
		}
	}

	/** Releases the vertex buffers RHI resource. */
	virtual void ReleaseRHI()
	{
		VertexBufferRHI.SafeRelease();
		SetBufferSize(0);
	}

	/** Returns a friendly name for this buffer. */
	virtual FString GetFriendlyName() const { return TEXT("SlateElementVertices"); }

	/** Returns the size of the buffer in bytes. */
	int32 GetBufferSize() const { return BufferSize; }

	/** Returns the used size of this buffer */
	int32 GetBufferUsageSize() const { return BufferUsageSize; }

	/** Resets the usage of the buffer */
	void ResetBufferUsage() { BufferUsageSize = 0; }

	/** Resizes buffer, accumulates states safely on render thread */
	void PreFillBuffer(FRHICommandListBase& RHICmdList, int32 RequiredVertexCount, bool bShrinkToMinSize)
	{
		SCOPE_CYCLE_COUNTER(STAT_SlatePreFullBufferTime);

		if (RequiredVertexCount > 0 )
		{
#if !SLATE_USE_32BIT_INDICES
			// make sure our index buffer can handle this
			checkf(RequiredVertexCount < 0xFFFF, TEXT("Slate vertex buffer is too large (%d) to work with uint16 indices"), RequiredVertexCount);
#endif
			int32 RequiredBufferSize = RequiredVertexCount*sizeof(VertexType);

			// resize if needed
			if (RequiredBufferSize > GetBufferSize() || bShrinkToMinSize)
			{
				ResizeBuffer(RHICmdList, RequiredBufferSize);
			}

			BufferUsageSize = RequiredBufferSize;
		}
	}

	int32 GetMinBufferSize() const { return MinBufferSize; }

private:
	/** Resizes the buffer to the passed in size.  Preserves internal data*/
	void ResizeBuffer(FRHICommandListBase& RHICmdList, int32 NewSizeBytes)
	{
		QUICK_SCOPE_CYCLE_COUNTER(Slate_RTResizeBuffer);

		int32 FinalSize = FMath::Max( NewSizeBytes, MinBufferSize );

		if (FinalSize != 0 && FinalSize != BufferSize)
		{
			VertexBufferRHI.SafeRelease();

			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateVertex(TEXT("SlateElementVertices"), FinalSize)
				.AddUsage(EBufferUsageFlags::Dynamic)
				.DetermineInitialState();

			VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);

			check(IsValidRef(VertexBufferRHI));

			SetBufferSize(FinalSize);
		}
	}

	void SetBufferSize(int32 NewBufferSize)
	{
		DEC_MEMORY_STAT_BY(STAT_SlateVertexBufferMemory, BufferSize);
		BufferSize = NewBufferSize;
		INC_MEMORY_STAT_BY(STAT_SlateVertexBufferMemory, BufferSize);
	}

private:
	/** The size of the buffer in bytes. */
	int32 BufferSize;
	
	/** The minimum size the buffer should always be */
	int32 MinBufferSize;

	/** The size of the used portion of the buffer */
	int32 BufferUsageSize;

	/** Hidden copy methods. */
	TSlateElementVertexBuffer( const TSlateElementVertexBuffer& );
	void operator=(const TSlateElementVertexBuffer& );
};


class FSlateStencilClipVertexBuffer : public FVertexBuffer
{
public:
	FSlateStencilClipVertexBuffer()
	{}

	~FSlateStencilClipVertexBuffer() = default;

	/** Initializes the vertex buffers RHI resource. */
	virtual void InitRHI(FRHICommandListBase& RHICmdList)
	{
		if (!IsValidRef(VertexBufferRHI))
		{
			const uint32 Verts[] = { 0, 1, 2, 3 };

			VertexBufferRHI = UE::RHIResourceUtils::CreateVertexBufferFromArray(RHICmdList, TEXT("SlateStencilClipVertexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(Verts));

			// Ensure the vertex buffer could be created
			check(IsValidRef(VertexBufferRHI));
		}
	}

	/** Releases the vertex buffers RHI resource. */
	virtual void ReleaseRHI()
	{
		VertexBufferRHI.SafeRelease();
	}

	/** Returns a friendly name for this buffer. */
	virtual FString GetFriendlyName() const { return TEXT("SlateElementVertices"); }
};

extern TGlobalResource<FSlateStencilClipVertexBuffer> GSlateStencilClipVertexBuffer;
