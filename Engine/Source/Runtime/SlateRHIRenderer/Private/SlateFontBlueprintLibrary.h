// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Fonts/SlateFontInfo.h"
#include "UObject/Class.h"
#include "SlateFontBlueprintLibrary.generated.h"

SLATERHIRENDERER_API DECLARE_LOG_CATEGORY_EXTERN(LogSlateFontBlueprint, Log, All);

UCLASS()
class USlateFontBlueprintLibrary : public UObject
{
	GENERATED_BODY()

	UFUNCTION(BlueprintPure, Category = "Slate Font", meta = (NativeMakeFunc))
//TODO: add deprecation in a second step to make migration process easier
// UFUNCTION(BlueprintPure, Category = "Slate Font", meta = (DeprecatedFunction, DeprecationMessage = "MakeSlateFontInfo from SlateFontBlueprintLibrary has been deprecated, use MakeSlateFontInfo from SlateFontInfoBlueprintLibrary instead."))
	static SLATERHIRENDERER_API FSlateFontInfo MakeSlateFontInfo(const UObject* FontObject, UObject* FontMaterial, FFontOutlineSettings OutlineSettings, FName TypefaceFontName, float Size = 24.0f, int32 LetterSpacing = 0, float SkewAmount = 0.0f, bool bForceMonospaced = false, bool bMaterialIsStencil = false, float MonospacedWidth = 1.0f);
};