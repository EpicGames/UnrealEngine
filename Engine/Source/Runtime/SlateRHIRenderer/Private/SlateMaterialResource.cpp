// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateMaterialResource.h"
#include "SlateRHIRendererSettings.h"
#include "Engine/Texture.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialShared.h"
#include "Styling/SlateBrush.h"

namespace SlateMaterialResource
{

#if SLATE_CHECK_UOBJECT_RENDER_RESOURCES
	void CheckInvalidUMaterial(const UMaterialInterface& InMaterialResource, const FName& InDebugName)
	{
		if (GSlateCheckUObjectRenderResources)
		{
			bool bIsValidLowLevel = InMaterialResource.IsValidLowLevelFast(false);
			if (!bIsValidLowLevel || !IsValid(&InMaterialResource) || InMaterialResource.GetClass() == UMaterialInterface::StaticClass())
			{
				UE_LOGF(LogSlate, Error, "Material '%ls' is not valid. PendingKill:'%d'. ValidLowLevelFast:'%d'. InvalidClass:'%d'"
					, *InDebugName.ToString()
					, (bIsValidLowLevel ? !IsValid(&InMaterialResource) : false)
					, bIsValidLowLevel
					, (bIsValidLowLevel ? InMaterialResource.GetClass() == UMaterialInterface::StaticClass() : false));

				const TCHAR* Message = TEXT("We detected an invalid resource in FSlateMaterialResource. Check the log for more detail.");
				if (GSlateCheckUObjectRenderResourcesShouldLogFatal)
				{
					UE_LOGF(LogSlate, Fatal, "%ls", Message);
				}
				else
				{
					ensureAlwaysMsgf(false, TEXT("%s"), Message);
				}
			}
		}
	}

	void CheckInvalidMaterialProxy(const FMaterialRenderProxy* MaterialProxy, const FName& InDebugName)
	{
		if (GSlateCheckUObjectRenderResources)
		{
			if (MaterialProxy == nullptr || MaterialProxy->IsDeleted() || MaterialProxy->IsMarkedForGarbageCollection())
			{
				UE_LOGF(LogSlate, Error, "Material '%ls' Render Proxy is: nullptr:'%d'. Deleted:'%d'. Marked for GC:'%d'"
					, *InDebugName.ToString()
					, (MaterialProxy == nullptr)
					, (MaterialProxy ? MaterialProxy->IsDeleted() : false)
					, (MaterialProxy ? MaterialProxy->IsMarkedForGarbageCollection() : false));

				const TCHAR* Message = TEXT("We detected an invalid resource render proxy in FSlateMaterialResource. Check the log for more detail.");
				if (GSlateCheckUObjectRenderResourcesShouldLogFatal)
				{
					UE_LOGF(LogSlate, Fatal, "%ls", Message);
				}
				else
				{
					ensureAlwaysMsgf(false, TEXT("%s"), Message);
				}
			}
		}
	}
#endif

}

FSlateMaterialResource::FSlateMaterialResource(const UMaterialInterface& InMaterialResource, const FVector2f InImageSize, FSlateShaderResource* InTextureMask )
	: MaterialObject( &InMaterialResource)
	, SlateProxy( new FSlateShaderResourceProxy )
	, TextureMaskResource( InTextureMask )
	, Width(FMath::RoundToInt(InImageSize.X))
	, Height(FMath::RoundToInt(InImageSize.Y))
	, CachedSlatePostBuffers(ESlatePostRT::None)
	, CachedUsesVirtualTextures(false)
{
#if SLATE_CHECK_UOBJECT_RENDER_RESOURCES
	SlateMaterialResource::CheckInvalidUMaterial(InMaterialResource, NAME_None);

	MaterialProxy = InMaterialResource.GetRenderProxy();

	MaterialObjectWeakPtr = MaterialObject;
	UpdateMaterialName();

	SlateMaterialResource::CheckInvalidMaterialProxy(MaterialProxy, DebugName);
#else
	MaterialProxy = InMaterialResource.GetRenderProxy();
#endif

	SlateProxy->ActualSize = InImageSize.IntPoint();
	SlateProxy->Resource = this;

	if (MaterialObject && (IsInRenderingThread() || IsInGameThread()))
	{
		TArray<UTexture*> OutUsedTextures;
		MaterialObject->GetUsedTextures(OutUsedTextures);

		CachedSlatePostBuffers = ESlatePostRT::None;

		if (const USlateRHIRendererSettings* RendererSettings = USlateRHIRendererSettings::Get())
		{
			for (const UTexture* OutUsedTexture : OutUsedTextures)
			{
				for (const TPair<ESlatePostRT, FSlatePostSettings>& SlatePostSetting : RendererSettings->GetSlatePostSettings())
				{
					const ESlatePostRT SlatePostBitflag = SlatePostSetting.Key;
					const FSlatePostSettings& SlatePostSettingValue = SlatePostSetting.Value;

					if (SlatePostSettingValue.bEnabled && OutUsedTexture && OutUsedTexture->GetPathName() == SlatePostSettingValue.GetPathToSlatePostRT())
					{
						CachedSlatePostBuffers |= SlatePostBitflag;
					}
				}

				CachedUsesVirtualTextures |= (OutUsedTexture && OutUsedTexture->IsCurrentlyVirtualTextured());
			}
		}
	}

	if (MaterialProxy && (MaterialProxy->IsDeleted() || MaterialProxy->IsMarkedForGarbageCollection()))
	{
		MaterialProxy = nullptr;
	}
}

FSlateMaterialResource::~FSlateMaterialResource()
{
	if (SlateProxy)
	{
		delete SlateProxy;
	}
}

ESlatePostRT FSlateMaterialResource::GetUsedSlatePostBuffers() const
{
	return CachedSlatePostBuffers;
}

bool FSlateMaterialResource::RequiresVirtualTextureFeedback() const
{
	return CachedUsesVirtualTextures;
}

bool FSlateMaterialResource::IsResourceValid() const
{
	if (MaterialProxy && MaterialObject)
	{
		// Conservatively only return invalid in scenarios where proxy is mismatched
		return MaterialProxy == MaterialObject->GetRenderProxy();
	}

	return true;
}

void FSlateMaterialResource::UpdateMaterial(const UMaterialInterface& InMaterialResource, const FVector2f InImageSize, FSlateShaderResource* InTextureMask)
{
	MaterialObject = ensure(IsValid(&InMaterialResource) && InMaterialResource.IsValidLowLevelFast(false)) ? &InMaterialResource : nullptr;
	MaterialProxy = MaterialObject ? MaterialObject->GetRenderProxy() : nullptr;

#if SLATE_CHECK_UOBJECT_RENDER_RESOURCES
	SlateMaterialResource::CheckInvalidUMaterial(InMaterialResource, DebugName);

	MaterialObjectWeakPtr = MaterialObject;
	UpdateMaterialName();

	SlateMaterialResource::CheckInvalidMaterialProxy(MaterialProxy, DebugName);
#endif

	if (MaterialObject && (IsInRenderingThread() || IsInGameThread()))
	{
		// Quality / Feature level irrelevant since flag to search all levels for both is true
		TArray<UTexture*> OutUsedTextures;
		MaterialObject->GetUsedTextures(OutUsedTextures);

		CachedSlatePostBuffers = ESlatePostRT::None;
		if (const USlateRHIRendererSettings* RendererSettings = USlateRHIRendererSettings::Get())
		{
			for (const UTexture* OutUsedTexture : OutUsedTextures)
			{
				for (const TPair<ESlatePostRT, FSlatePostSettings>& SlatePostSetting : RendererSettings->GetSlatePostSettings())
				{
					const ESlatePostRT SlatePostBitflag = SlatePostSetting.Key;
					const FSlatePostSettings& SlatePostSettingValue = SlatePostSetting.Value;

					if (SlatePostSettingValue.bEnabled && OutUsedTexture && OutUsedTexture->GetPathName() == SlatePostSettingValue.GetPathToSlatePostRT())
					{
						CachedSlatePostBuffers |= SlatePostBitflag;
					}
				}
			}
		}
	}

	if (MaterialProxy && (MaterialProxy->IsDeleted() || MaterialProxy->IsMarkedForGarbageCollection()))
	{
		MaterialProxy = nullptr;
	}

	if (!SlateProxy)
	{
		SlateProxy = new FSlateShaderResourceProxy;
	}

	TextureMaskResource = InTextureMask;

	SlateProxy->ActualSize = InImageSize.IntPoint();
	SlateProxy->Resource = this;

	Width = FMath::RoundToInt(InImageSize.X);
	Height = FMath::RoundToInt(InImageSize.Y);
}

void FSlateMaterialResource::ResetMaterial()
{
	MaterialObject = nullptr;
	MaterialProxy = nullptr;

#if SLATE_CHECK_UOBJECT_RENDER_RESOURCES
	MaterialObjectWeakPtr = nullptr;
	UpdateMaterialName();
#endif

	CachedSlatePostBuffers = ESlatePostRT::None;

	TextureMaskResource = nullptr;
	if (SlateProxy)
	{
		delete SlateProxy;
	}
	SlateProxy = nullptr;
	Width = 0;
	Height = 0;
}

#if SLATE_CHECK_UOBJECT_RENDER_RESOURCES
void FSlateMaterialResource::UpdateMaterialName()
{
	const UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(MaterialObject);
	if(MID && MID->Parent)
	{
		// MID's don't have nice names. Get the name of the parent instead for tracking
		DebugName = MID->Parent->GetFName();
	}
	else if(MaterialObject)
	{
		DebugName = MaterialObject->GetFName();
	}
	else
	{
		DebugName = NAME_None;
	}
}

void FSlateMaterialResource::CheckForStaleResources() const
{
	if (DebugName != NAME_None)
	{
		// pending kill objects may still be rendered for a frame so it is valid for the check to pass
		const bool bEvenIfPendingKill = true;
		// This test needs to be thread safe.  It doesn't give us as many chances to trap bugs here but it is still useful
		const bool bThreadSafe = true;
		checkf(MaterialObjectWeakPtr.IsValid(bEvenIfPendingKill, bThreadSafe), TEXT("Material %s has become invalid.  This means the resource was garbage collected while slate was using it"), *DebugName.ToString());
	}
}

#endif
