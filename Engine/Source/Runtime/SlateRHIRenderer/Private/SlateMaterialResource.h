// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Textures/SlateShaderResource.h"
#include "Materials/MaterialInterface.h"

class FMaterialRenderProxy;

/**
 * A resource for rendering a UMaterial in Slate
 */
class FSlateMaterialResource : public FSlateShaderResource
{
public:
	FSlateMaterialResource(const UMaterialInterface& InMaterialResource, const FVector2f InImageSize, FSlateShaderResource* InTextureMask = nullptr );
	~FSlateMaterialResource();

	//~ Begin FSlateShaderResource Interface.
	virtual uint32 GetWidth() const override { return Width; }
	virtual uint32 GetHeight() const override { return Height; }
	virtual ESlateShaderResource::Type GetType() const override { return ESlateShaderResource::Material; }
	virtual ESlatePostRT GetUsedSlatePostBuffers() const override;
	virtual bool RequiresVirtualTextureFeedback() const override;
	virtual bool IsResourceValid() const override;
	//~ End FSlateShaderResource Interface.

	void UpdateMaterial(const UMaterialInterface& InMaterialResource, const FVector2f InImageSize, FSlateShaderResource* InTextureMask );
	void ResetMaterial();

	/** @return The material render proxy */
	FMaterialRenderProxy* GetRenderProxy() const { return MaterialProxy; }

	/** @return the material object */
	const UMaterialInterface* GetMaterialObject() const
	{
		return MaterialObject;
	}

	/** Slate proxy used for batching the material */
	FSlateShaderResourceProxy* GetResourceProxy() const { return SlateProxy; }

	FSlateShaderResource* GetTextureMaskResource() const { return TextureMaskResource; }

#if SLATE_CHECK_UOBJECT_RENDER_RESOURCES
	virtual void CheckForStaleResources() const override;
#endif

private:
	const class UMaterialInterface* MaterialObject;
	class FMaterialRenderProxy* MaterialProxy;

#if SLATE_CHECK_UOBJECT_RENDER_RESOURCES
	// Used to guard against crashes when the material object is deleted.  This is expensive so we do not do it in shipping
	TWeakObjectPtr<const UMaterialInterface> MaterialObjectWeakPtr;
	FName DebugName;
#endif

	/** Slate proxy used for batching the material */
	FSlateShaderResourceProxy* SlateProxy;

	FSlateShaderResource* TextureMaskResource;
	uint32 Width;
	uint32 Height;

	/** Cached slate SlatePostRT assets / buffers in use */
	ESlatePostRT CachedSlatePostBuffers;

	/** Cached result of whether we use virtual textures */
	bool CachedUsesVirtualTextures;

private:
#if SLATE_CHECK_UOBJECT_RENDER_RESOURCES
	void UpdateMaterialName();
#endif
};

