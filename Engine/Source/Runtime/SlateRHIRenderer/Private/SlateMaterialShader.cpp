// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateMaterialShader.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "RHIStaticStates.h"
#include "SceneView.h"
#include "ShaderParameterUtils.h"
#include "MeshDrawShaderBindings.h"
#include "SceneInterface.h"

IMPLEMENT_TYPE_LAYOUT(FSlateMaterialShaderVS);
IMPLEMENT_TYPE_LAYOUT(FSlateMaterialShaderPS);

FSlateMaterialShaderVS::FSlateMaterialShaderVS(const FMaterialShaderType::CompiledShaderInitializerType& Initializer)
	: FMaterialShader(Initializer)
{}

void FSlateMaterialShaderVS::ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	// Set defines based on what this shader will be used for
	OutEnvironment.SetDefine( TEXT("USE_MATERIALS"), 1 );
	OutEnvironment.SetDefine( TEXT("NUM_CUSTOMIZED_UVS"), Parameters.MaterialParameters.NumCustomizedUVs );
	OutEnvironment.SetDefine(TEXT("HAS_SCREEN_POSITION"), (bool)Parameters.MaterialParameters.bHasVertexPositionOffsetConnected);

	FMaterialShader::ModifyCompilationEnvironment( Parameters, OutEnvironment );
}

bool FSlateMaterialShaderVS::ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
{
	return Parameters.MaterialParameters.MaterialDomain == MD_UI;
}

void FSlateMaterialShaderVS::SetMaterialShaderParameters(
	FMeshDrawSingleShaderBindings& ShaderBindings,
	const FSceneInterface* Scene,
	const TUniformBufferRef<FViewUniformShaderParameters>& ViewUniformBuffer,
	const FMaterialRenderProxy* MaterialRenderProxy,
	const FMaterial* Material)
{
	const auto& ViewUniformBufferParameter = GetUniformBufferParameter<FViewUniformShaderParameters>();
	ShaderBindings.Add(ViewUniformBufferParameter, ViewUniformBuffer);

	const ERHIFeatureLevel::Type FeatureLevel = Scene ? Scene->GetFeatureLevel() : GMaxRHIFeatureLevel;
	FMaterialShader::GetShaderBindings(Scene, FeatureLevel, *MaterialRenderProxy, *Material, ShaderBindings);
}

bool FSlateMaterialShaderPS::ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
{
	return Parameters.MaterialParameters.MaterialDomain == MD_UI;
}


void FSlateMaterialShaderPS::ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	// Set defines based on what this shader will be used for
	OutEnvironment.SetDefine( TEXT("USE_MATERIALS"), 1 );
	OutEnvironment.SetDefine( TEXT("NUM_CUSTOMIZED_UVS"), Parameters.MaterialParameters.NumCustomizedUVs);

	FMaterialShader::ModifyCompilationEnvironment( Parameters, OutEnvironment );
}

FSlateMaterialShaderPS::FSlateMaterialShaderPS(const FMaterialShaderType::CompiledShaderInitializerType& Initializer)
	: FMaterialShader(Initializer)
{
	ShaderParams.Bind(Initializer.ParameterMap, TEXT("ShaderParams"));
	ShaderParams2.Bind(Initializer.ParameterMap, TEXT("ShaderParams2"));
	GammaAndAlphaValues.Bind(Initializer.ParameterMap, TEXT("GammaAndAlphaValues"));
	DrawFlags.Bind(Initializer.ParameterMap, TEXT("DrawFlags"));
	AdditionalTextureParameter.Bind(Initializer.ParameterMap, TEXT("ElementTexture"));
	TextureParameterSampler.Bind(Initializer.ParameterMap, TEXT("ElementTextureSampler"));
}

void FSlateMaterialShaderPS::SetMaterialShaderParameters(
	FMeshDrawSingleShaderBindings& ShaderBindings,
	const FSceneInterface* Scene,
	const TUniformBufferRef<FViewUniformShaderParameters>& ViewUniformBuffer,
	const FMaterialRenderProxy* MaterialRenderProxy,
	const FMaterial* Material,
	const FShaderParams& InShaderParams)
{
	ShaderBindings.Add(ShaderParams,  (FVector4f)InShaderParams.PixelParams);
	ShaderBindings.Add(ShaderParams2, (FVector4f)InShaderParams.PixelParams2);

	const auto& ViewUniformBufferParameter = GetUniformBufferParameter<FViewUniformShaderParameters>();
	ShaderBindings.Add(ViewUniformBufferParameter, ViewUniformBuffer);

	const ERHIFeatureLevel::Type FeatureLevel = Scene ? Scene->GetFeatureLevel() : GMaxRHIFeatureLevel;
	FMaterialShader::GetShaderBindings(Scene, FeatureLevel, *MaterialRenderProxy, *Material, ShaderBindings);
}

void FSlateMaterialShaderPS::SetAdditionalTexture(FMeshDrawSingleShaderBindings& ShaderBindings, FRHITexture* InTexture, const FSamplerStateRHIRef SamplerState)
{
	ShaderBindings.AddTexture(AdditionalTextureParameter, TextureParameterSampler, SamplerState, InTexture);
}

void FSlateMaterialShaderPS::SetDisplayGammaAndContrast(FMeshDrawSingleShaderBindings& ShaderBindings, float InDisplayGamma, float InContrast)
{
	FVector4f InGammaValues(2.2f / InDisplayGamma, 1.0f / InDisplayGamma, 0.0f, InContrast);

	ShaderBindings.Add(GammaAndAlphaValues, InGammaValues);
}

void FSlateMaterialShaderPS::SetDrawFlags(FMeshDrawSingleShaderBindings& ShaderBindings, ESlateShader InShaderType, bool bDrawDisabledEffect)
{
	FUintVector4 InDrawFlags((bDrawDisabledEffect ? 1u : 0u), uint32(InShaderType), 0, 0);

	ShaderBindings.Add(DrawFlags, InDrawFlags);
}

#define IMPLEMENT_SLATE_VERTEXMATERIALSHADER_TYPE(bUseInstancing) \
	typedef TSlateMaterialShaderVS<bUseInstancing> TSlateMaterialShaderVS##bUseInstancing; \
	IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, TSlateMaterialShaderVS##bUseInstancing, TEXT("/Engine/Private/SlateVertexShader.usf"), TEXT("Main"), SF_Vertex);

/** Instancing vertex shader */
IMPLEMENT_SLATE_VERTEXMATERIALSHADER_TYPE(true);
/** Non instancing vertex shader */
IMPLEMENT_SLATE_VERTEXMATERIALSHADER_TYPE(false);

#define IMPLEMENT_SLATE_MATERIALSHADER_TYPE(ShaderType) \
	typedef TSlateMaterialShaderPS<ESlateShader::ShaderType> TSlateMaterialShaderPS##ShaderType; \
	IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, TSlateMaterialShaderPS##ShaderType, TEXT("/Engine/Private/SlateElementPixelShader.usf"), TEXT("Main"), SF_Pixel);

IMPLEMENT_SLATE_MATERIALSHADER_TYPE(Custom)
IMPLEMENT_SLATE_MATERIALSHADER_TYPE(Default);
IMPLEMENT_SLATE_MATERIALSHADER_TYPE(Border);
IMPLEMENT_SLATE_MATERIALSHADER_TYPE(GrayscaleFont);
IMPLEMENT_SLATE_MATERIALSHADER_TYPE(ColorFont);
IMPLEMENT_SLATE_MATERIALSHADER_TYPE(RoundedBox);
IMPLEMENT_SLATE_MATERIALSHADER_TYPE(SdfFont);
IMPLEMENT_SLATE_MATERIALSHADER_TYPE(MsdfFont);
IMPLEMENT_SLATE_MATERIALSHADER_TYPE(Dynamic);
