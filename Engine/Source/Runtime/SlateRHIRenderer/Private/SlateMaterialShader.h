// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Rendering/RenderingCommon.h"
#include "ShaderParameters.h"
#include "Shader.h"
#include "MaterialShaderType.h"
#include "MaterialShader.h"

class FMeshDrawSingleShaderBindings;

class FSlateMaterialShaderVS : public FMaterialShader
{
	DECLARE_TYPE_LAYOUT(FSlateMaterialShaderVS, NonVirtual);
public:
	FSlateMaterialShaderVS() {}
	FSlateMaterialShaderVS(const FMaterialShaderType::CompiledShaderInitializerType& Initializer);

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters);

	void SetMaterialShaderParameters(
		FMeshDrawSingleShaderBindings& ShaderBindings,
		const FSceneInterface* Scene,
		const TUniformBufferRef<FViewUniformShaderParameters>& ViewUniformBuffer,
		const FMaterialRenderProxy* MaterialRenderProxy,
		const FMaterial* Material);
};

class FSlateMaterialShaderPS : public FMaterialShader
{
	DECLARE_TYPE_LAYOUT(FSlateMaterialShaderPS, NonVirtual);
public:

	/** Only compile shaders used with UI. */
	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters);

	/** Modifies the compilation of this shader. */
	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	FSlateMaterialShaderPS() {}
	FSlateMaterialShaderPS(const FMaterialShaderType::CompiledShaderInitializerType& Initializer);

	void SetMaterialShaderParameters(
		FMeshDrawSingleShaderBindings& ShaderBindings,
		const FSceneInterface* Scene,
		const TUniformBufferRef<FViewUniformShaderParameters>& ViewUniformBuffer,
		const FMaterialRenderProxy* MaterialRenderProxy,
		const FMaterial* Material,
		const FShaderParams& InShaderParams);

	void SetDisplayGammaAndContrast(FMeshDrawSingleShaderBindings& ShaderBindings, float InDisplayGamma, float InContrast);

	void SetAdditionalTexture(FMeshDrawSingleShaderBindings& ShaderBindings, FRHITexture* InTexture, const FSamplerStateRHIRef SamplerState );

	void SetDrawFlags(FMeshDrawSingleShaderBindings& ShaderBindings, ESlateShader InShaderType, bool bDrawDisabledEffect);

private:
	LAYOUT_FIELD(FShaderParameter, GammaAndAlphaValues);
	LAYOUT_FIELD(FShaderParameter, DrawFlags);
	LAYOUT_FIELD(FShaderParameter, ShaderParams);
	LAYOUT_FIELD(FShaderParameter, ShaderParams2);
	/** Extra texture (like a font atlas) to be used in addition to any material textures */
	LAYOUT_FIELD(FShaderResourceParameter, TextureParameterSampler);
	LAYOUT_FIELD(FShaderResourceParameter, AdditionalTextureParameter);
};

template<bool bUseInstancing>
class TSlateMaterialShaderVS : public FSlateMaterialShaderVS
{
public:
	DECLARE_SHADER_TYPE(TSlateMaterialShaderVS,Material);

	TSlateMaterialShaderVS() { }

	TSlateMaterialShaderVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FSlateMaterialShaderVS( Initializer )
	{ }
	
	/** Only compile shaders used with UI. */
	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
	{
		return FSlateMaterialShaderVS::ShouldCompilePermutation(Parameters);
	}

	/** Modifies the compilation of this shader. */
	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FSlateMaterialShaderVS::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("USE_SLATE_INSTANCING"), (uint32)( bUseInstancing ? 1 : 0 ));
	}
};

bool IsValidSlateMaterialShaderType(ESlateShader ShaderType, EShaderPlatform ShaderPlatform);

template<ESlateShader ShaderType>
class TSlateMaterialShaderPS : public FSlateMaterialShaderPS
{
public:
	DECLARE_SHADER_TYPE(TSlateMaterialShaderPS,Material);

	TSlateMaterialShaderPS() { }

	TSlateMaterialShaderPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FSlateMaterialShaderPS( Initializer )
	{ }
	
	/** Only compile shaders used with UI. */
	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
	{
		return FSlateMaterialShaderPS::ShouldCompilePermutation(Parameters) && IsValidSlateMaterialShaderType(ShaderType, Parameters.Platform);
	}

	/** Modifies the compilation of this shader. */
	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FSlateMaterialShaderPS::ModifyCompilationEnvironment(Parameters,OutEnvironment);

		OutEnvironment.SetDefine(TEXT("SHADER_TYPE"), (uint32)ShaderType);
	}
};
