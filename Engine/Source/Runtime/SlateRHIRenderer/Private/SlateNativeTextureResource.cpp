// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateNativeTextureResource.h"
#include "Textures/SlateShaderResource.h"
#include "Slate/SlateTextures.h"

FSlateDynamicTextureResource::FSlateDynamicTextureResource(FSlateTexture2DRHIRef* ExistingTexture)
	: Proxy(new FSlateShaderResourceProxy)
	, RHIRefTexture(ExistingTexture != NULL ? ExistingTexture : new FSlateTexture2DRHIRef(NULL, 0, 0))
{
	Proxy->Resource = RHIRefTexture;
}

FSlateDynamicTextureResource::~FSlateDynamicTextureResource()
{
	if(Proxy)
	{
		delete Proxy;
	}

	if(RHIRefTexture)
	{
		delete RHIRefTexture;
	}
}
