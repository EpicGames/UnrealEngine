// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FSlateShaderResourceProxy;
class FSlateTexture2DRHIRef;

/** Represents a dynamic texture resource for rendering in Slate*/
class FSlateDynamicTextureResource
{
public:
	FSlateDynamicTextureResource(FSlateTexture2DRHIRef* ExistingTexture);
	~FSlateDynamicTextureResource();

	FSlateShaderResourceProxy* Proxy;
	FSlateTexture2DRHIRef* RHIRefTexture;
};
