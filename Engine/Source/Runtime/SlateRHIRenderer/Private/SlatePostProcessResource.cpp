// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlatePostProcessResource.h"
#include "RenderUtils.h"
#include "RHI.h"
#include "RHICommandList.h"

DECLARE_MEMORY_STAT(TEXT("PostProcess RenderTargets"), STAT_SLATEPPRenderTargetMem, STATGROUP_SlateMemory);

FSlatePostProcessResource::FSlatePostProcessResource(int32 InRenderTargetCount)
	: PixelFormat(PF_Unknown)
	, RenderTargetSize(FIntPoint::ZeroValue)
	, RenderTargetCount(InRenderTargetCount)
{
	FrameUsed = GFrameCounter;
}

FSlatePostProcessResource::~FSlatePostProcessResource()
{

}

void FSlatePostProcessResource::Update(const FIntPoint& NewSize, EPixelFormat RequestedPixelFormat)
{
	if(NewSize.X > RenderTargetSize.X || NewSize.Y > RenderTargetSize.Y || RenderTargetSize == FIntPoint::ZeroValue || RenderTargets.Num() == 0 || PixelFormat != RequestedPixelFormat)
	{
		FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

		if(!IsInitialized())
		{
			InitResource(RHICmdList);
		}

		FIntPoint NewMaxSize(FMath::Max(NewSize.X, RenderTargetSize.X), FMath::Max(NewSize.Y, RenderTargetSize.Y));
		ResizeTargets(RHICmdList, NewMaxSize, RequestedPixelFormat);
	}
	FrameUsed = GFrameCounter;
}

void FSlatePostProcessResource::ResizeTargets(FRHICommandListBase& RHICmdList, const FIntPoint& NewSize, EPixelFormat RequestedPixelFormat)
{
	check(IsInRenderingThread());

	RenderTargets.Empty();

	RenderTargetSize = NewSize;
	PixelFormat = RequestedPixelFormat;
	if (RenderTargetSize.X > 0 && RenderTargetSize.Y > 0)
	{
		for (int32 TexIndex = 0; TexIndex < RenderTargetCount; ++TexIndex)
		{
			const FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc::Create2D(TEXT("FSlatePostProcessResource"))
				.SetExtent(RenderTargetSize)
				.SetFormat(PixelFormat)
				.SetFlags(ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource)
				.SetInitialState(ERHIAccess::SRVMask);

			FTextureRHIRef RenderTargetTextureRHI = RHICmdList.CreateTexture(Desc);

			RenderTargets.Add(RenderTargetTextureRHI);
		}
	}

	STAT(int64 TotalMemory = RenderTargetCount * GPixelFormats[PixelFormat].BlockBytes*RenderTargetSize.X*RenderTargetSize.Y);
	SET_MEMORY_STAT(STAT_SLATEPPRenderTargetMem, TotalMemory);
}

void FSlatePostProcessResource::CleanUp()
{
	BeginReleaseResource(this);

	BeginCleanup(this);
}

void FSlatePostProcessResource::InitRHI(FRHICommandListBase& RHICmdList)
{
}

void FSlatePostProcessResource::ReleaseRHI()
{
	SET_MEMORY_STAT(STAT_SLATEPPRenderTargetMem, 0);

	RenderTargetSize = FIntPoint::ZeroValue;

	RenderTargets.Empty();
}

