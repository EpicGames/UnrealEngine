// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Textures/SlateShaderResource.h"
#include "RenderResource.h"
#include "RenderDeferredCleanup.h"

/**
 * Handle to resources used for post processing.  This should not be deleted manually because it implements FDeferredCleanupInterface
 */
class FSlatePostProcessResource : public FSlateShaderResource, public FRenderResource, private FDeferredCleanupInterface
{
public:
	FSlatePostProcessResource(int32 InRenderTargetCount);
	~FSlatePostProcessResource();

	const FTextureRHIRef& GetRenderTarget(int32 Index)
	{
		return RenderTargets[Index]; 
	}

	/** Performs per frame updates to this resource */
	void Update(const FIntPoint& NewSize, EPixelFormat RequestedPixelFormat);

	void CleanUp();

	/** FRenderResource interface */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;

	/** FSlateShaderResource interface */
	virtual uint32 GetWidth() const override { return RenderTargetSize.X; }
	virtual uint32 GetHeight() const override { return RenderTargetSize.Y; }
	virtual ESlateShaderResource::Type GetType() const override { return ESlateShaderResource::PostProcess; }

	EPixelFormat GetPixelFormat() const { return PixelFormat; }
	uint64 GetFrameUsed() { return FrameUsed; }

private:
	/** Resizes targets to the new size */
	void ResizeTargets(FRHICommandListBase& RHICmdList, const FIntPoint& NewSize, EPixelFormat RequestedPixelFormat);

private:
	TArray<FTextureRHIRef, TInlineAllocator<2>> RenderTargets;
	EPixelFormat PixelFormat;
	FIntPoint RenderTargetSize;
	int32 RenderTargetCount;
	uint64 FrameUsed;
};

