// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlatePostProcessor.h"
#include "SlateShaders.h"
#include "RendererUtils.h"
#include "Interfaces/SlateRHIRenderingPolicyInterface.h"

//////////////////////////////////////////////////////////////////////////

CSV_DECLARE_CATEGORY_MODULE_EXTERN(SLATECORE_API, Slate);

static int32 SlatePostBlurDualKawaseFilterEnable = 1;
static float SlatePostBlurStrengthOverride = 0.f;

FAutoConsoleVariableRef CVarSlatePostBlurUseDualKawaseFilter(
	TEXT("UI.SlatePostBlurUseDualKawaseFilter"),
	SlatePostBlurDualKawaseFilterEnable,
	TEXT("Toggles between the old Gaussian blur implementation (0) and the new optimized Dual Kawase filter blur implementation (1)"));

FAutoConsoleVariableRef CVarSlatePostBlurStrengthOverride(
	TEXT("UI.SlatePostBlurStrengthOverride"),
	SlatePostBlurStrengthOverride,
	TEXT("Globally overrides the blur strength for all Slate post process blurs (0 = no override)"));

float GetSlateHDRUILevel()
{
	static auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.HDR.UI.Level"));
	return CVar ? CVar->GetFloat() : 1.0f;
}

float GetSlateHDRUILuminance()
{
	static auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.HDR.UI.Luminance"));
	return CVar ? CVar->GetFloat() : 203.f;
}

int32 GetSlateHDRUILuminanceMode()
{
	static auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.HDR.UI.Luminance.Mode"));
	return CVar ? CVar->GetInt() : 1;
}

int32 GetSlateHDRUICompositeEOTF()
{
	static auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.HDR.UI.CompositeEOTF"));
	return CVar ? CVar->GetInt() : 0;
}

float GetSlateHDRUICompositeEOTFGamma()
{
	static auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.HDR.UI.CompositeEOTF.Gamma"));
	return CVar ? CVar->GetFloat() : 2.4f;
}

int32 GetSlateHDRUICompositeBlend()
{
	static auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.HDR.UI.CompositeBlend"));
	return CVar ? CVar->GetInt() : 0;
}

float GetSlateHDRUICompositeBlendGamma()
{
	static auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.HDR.UI.CompositeBlend.Gamma"));
	return CVar ? CVar->GetFloat() : 2.4f;
}

ETextureCreateFlags GetSlateTransientRenderTargetFlags()
{
	return ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource | ETextureCreateFlags::FastVRAM
		// Avoid fast clear metadata when this flag is set, since we'd otherwise have to clear transient render targets instead of discard.
#if PLATFORM_REQUIRES_TYPELESS_RESOURCE_DISCARD_WORKAROUND
		| ETextureCreateFlags::NoFastClear
#endif
		;
}

ETextureCreateFlags GetSlateTransientDepthStencilFlags()
{
	return ETextureCreateFlags::DepthStencilTargetable | ETextureCreateFlags::FastVRAM;
}

//////////////////////////////////////////////////////////////////////////

// Pixel shader to composite UI over HDR buffer prior to doing a blur
class FCompositeHDRForBlurPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FCompositeHDRForBlurPS);
	SHADER_USE_PARAMETER_STRUCT(FCompositeHDRForBlurPS, FGlobalShader);

	class FUseSRGBEncoding : SHADER_PERMUTATION_BOOL("SCRGB_ENCODING");
	using FPermutationDomain = TShaderPermutationDomain<FUseSRGBEncoding>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, UITexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, UIWriteMaskTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, UISampler)
		SHADER_PARAMETER(float, UILevel)
		SHADER_PARAMETER(float, UILuminance)
		SHADER_PARAMETER(int32, UICompositeEOTF)
		SHADER_PARAMETER(float, UICompositeEOTFGamma)
		SHADER_PARAMETER(int32, UICompositeBlend)
		SHADER_PARAMETER(float, UICompositeBlendGamma)
		SHADER_PARAMETER(float, UICompositeBlendInvGamma)
		SHADER_PARAMETER(FVector2f, UITextureSize)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) && (RHISupportsGeometryShaders(Parameters.Platform) || RHISupportsVertexShaderLayer(Parameters.Platform));
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("COMPOSITE_UI_FOR_BLUR_PS"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FCompositeHDRForBlurPS, "/Engine/Private/CompositeUIPixelShader.usf", "CompositeUIForBlur", SF_Pixel);

struct FSlateCompositeHDRForBlurPassInputs
{
	FIntRect InputRect;
	FRDGTexture* InputCompositeTexture;
	FRDGTexture* InputTexture;
	FIntPoint OutputExtent;
	float HDRPaperWhiteInNits;
};

FScreenPassTexture AddSlateCompositeHDRForBlurPass(FRDGBuilder& GraphBuilder, const FSlateCompositeHDRForBlurPassInputs& Inputs)
{
	const ERHIFeatureLevel::Type FeatureLevel = GMaxRHIFeatureLevel;
	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(FeatureLevel);
	FRDGTexture* UIWriteMaskTexture = nullptr;

	if (RHISupportsRenderTargetWriteMask(GMaxRHIShaderPlatform))
	{
		FRenderTargetWriteMask::Decode(GraphBuilder, ShaderMap, MakeArrayView({ Inputs.InputCompositeTexture }), UIWriteMaskTexture, TexCreate_None, TEXT("UIRTWriteMask"));
	}

	FScreenPassRenderTarget Output(
		GraphBuilder.CreateTexture(
			FRDGTextureDesc::Create2D(Inputs.OutputExtent, PF_FloatR11G11B10, FClearValueBinding::Black, GetSlateTransientRenderTargetFlags()),
			TEXT("CompositeHDRUI")),
		ERenderTargetLoadAction::ENoAction);

	const FScreenPassTextureViewport InputViewport = FScreenPassTextureViewport(Inputs.InputCompositeTexture, Inputs.InputRect);
	const FScreenPassTextureViewport OutputViewport = FScreenPassTextureViewport(Output);

	FCompositeHDRForBlurPS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FCompositeHDRForBlurPS::FUseSRGBEncoding>(Inputs.InputTexture->Desc.Format == PF_FloatRGBA);

	FCompositeHDRForBlurPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCompositeHDRForBlurPS::FParameters>();
	PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
	PassParameters->SceneTexture = Inputs.InputTexture;
	PassParameters->UITexture = Inputs.InputCompositeTexture;
	PassParameters->UIWriteMaskTexture = UIWriteMaskTexture;
	PassParameters->UISampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
	PassParameters->UITextureSize = InputViewport.Extent;
	PassParameters->UILevel = GetSlateHDRUILevel();
	PassParameters->UILuminance = GetSlateHDRUILuminanceMode() ? GetSlateHDRUILuminance() : Inputs.HDRPaperWhiteInNits;
	PassParameters->UICompositeEOTF = GetSlateHDRUICompositeEOTF();
	PassParameters->UICompositeEOTFGamma = GetSlateHDRUICompositeEOTFGamma();
	PassParameters->UICompositeBlend = GetSlateHDRUICompositeBlend();
	const float UICompositeBlendGamma = GetSlateHDRUICompositeBlendGamma();
	PassParameters->UICompositeBlendGamma = UICompositeBlendGamma;
	PassParameters->UICompositeBlendInvGamma = 1.f / UICompositeBlendGamma;

	TShaderMapRef<FCompositeHDRForBlurPS> PixelShader(ShaderMap, PermutationVector);
	AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("CompositeHDR"), FeatureLevel, OutputViewport, InputViewport, PixelShader, PassParameters);
	return Output;
}

//////////////////////////////////////////////////////////////////////////

class FSlatePostProcessDirectResamplePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FSlatePostProcessDirectResamplePS);
	SHADER_USE_PARAMETER_STRUCT(FSlatePostProcessDirectResamplePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ElementTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, ElementTextureSampler)
		SHADER_PARAMETER(FVector4f, UVBounds)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FSlatePostProcessDirectResamplePS, "/Engine/Private/SlatePostProcessPixelShader.usf", "Resample1Main", SF_Pixel);

class FSlatePostProcessResample2x2PS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FSlatePostProcessResample2x2PS);
	SHADER_USE_PARAMETER_STRUCT(FSlatePostProcessResample2x2PS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ElementTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, ElementTextureSampler)
		SHADER_PARAMETER(FVector4f, ShaderParams)
		SHADER_PARAMETER(FVector4f, UVBounds)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FSlatePostProcessResample2x2PS, "/Engine/Private/SlatePostProcessPixelShader.usf", "Resample2x2Main", SF_Pixel);

struct FSlatePostProcessDownsamplePassInputs
{
	FScreenPassTexture InputTexture;
	FIntPoint OutputExtent;
	uint32 Downscale;
};

FScreenPassTexture AddSlatePostProcessDownsamplePass(FRDGBuilder& GraphBuilder, const FSlatePostProcessDownsamplePassInputs& Inputs)
{
	const ERHIFeatureLevel::Type FeatureLevel = GMaxRHIFeatureLevel;
	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(FeatureLevel);

	const FScreenPassRenderTarget Output(
		GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(Inputs.OutputExtent, Inputs.InputTexture.Texture->Desc.Format, FClearValueBinding::None, GetSlateTransientRenderTargetFlags()), TEXT("DownsampleUI")),
		ERenderTargetLoadAction::ENoAction);

	const FScreenPassTextureViewport InputViewport(Inputs.InputTexture);
	// This ensures that if input resolution is not divisible by the Downscale, we actually scale exactly Downscale x Downscale pixels into one and pad the last row and column rather than just fitting to the rounded dimensions.
	const FScreenPassTextureViewport ProportionalInputViewport(Inputs.InputTexture.Texture, FIntRect(Inputs.InputTexture.ViewRect.Min, Inputs.InputTexture.ViewRect.Min + FIntPoint(Inputs.Downscale) * Inputs.OutputExtent));
	// Input parameters are still computed from actual InputViewport, otherwise pixels from outside the viewport get blended into the last row and column.
	const FScreenPassTextureViewportParameters InputParameters = GetScreenPassTextureViewportParameters(InputViewport);
	const FScreenPassTextureViewport OutputViewport(Output);

	if (Inputs.Downscale <= 2) // Only take 1 sample for 2x downscale
	{
		TShaderMapRef<FSlatePostProcessDirectResamplePS> PixelShader(ShaderMap);
		FSlatePostProcessDirectResamplePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSlatePostProcessDirectResamplePS::FParameters>();
		PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
		PassParameters->ElementTexture = Inputs.InputTexture.Texture;
		PassParameters->ElementTextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		PassParameters->UVBounds = FVector4f(InputParameters.UVViewportBilinearMin, InputParameters.UVViewportBilinearMax);
		AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("DownsampleUI"), FeatureLevel, OutputViewport, ProportionalInputViewport, PixelShader, PassParameters);
	}
	else // 4 samples for >2x downscale (not enough for >4x !)
	{
		const float OffsetFactor = Inputs.Downscale == 3 ? 2.f/3.f : 1.f;
		TShaderMapRef<FSlatePostProcessResample2x2PS> PixelShader(ShaderMap);
		FSlatePostProcessResample2x2PS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSlatePostProcessResample2x2PS::FParameters>();
		PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
		PassParameters->ElementTexture = Inputs.InputTexture.Texture;
		PassParameters->ElementTextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		PassParameters->ShaderParams = FVector4f(OffsetFactor * InputParameters.ExtentInverse.X, OffsetFactor * InputParameters.ExtentInverse.Y, 0.0f, 0.0f);
		PassParameters->UVBounds = FVector4f(InputParameters.UVViewportBilinearMin, InputParameters.UVViewportBilinearMax);
		AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("DownsampleUI"), FeatureLevel, OutputViewport, ProportionalInputViewport, PixelShader, PassParameters);
	}

	return Output;
}

//////////////////////////////////////////////////////////////////////////

enum class ESlatePostProcessUpsampleOutputFormat
{
	SDR = 0,
	HDR_SCRGB,
	HDR_PQ10,
	MAX
};

class FSlatePostProcessUpsamplePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FSlatePostProcessUpsamplePS);
	SHADER_USE_PARAMETER_STRUCT(FSlatePostProcessUpsamplePS, FGlobalShader);

	class FUpsampleOutputFormat : SHADER_PERMUTATION_ENUM_CLASS("UPSAMPLE_OUTPUT_FORMAT", ESlatePostProcessUpsampleOutputFormat);
	using FPermutationDomain = TShaderPermutationDomain<FUpsampleOutputFormat>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ElementTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, ElementTextureSampler)
		SHADER_PARAMETER(FVector4f, ShaderParams)
		SHADER_PARAMETER(FVector4f, ShaderParams2)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FSlatePostProcessUpsamplePS, "/Engine/Private/SlatePostProcessPixelShader.usf", "UpsampleMain", SF_Pixel);

struct FSlatePostProcessUpsampleInputs
{
	FScreenPassTexture InputTexture;
	FRDGTexture* OutputTextureToClear = nullptr;
	FRDGTexture* OutputTexture = nullptr;
	ERenderTargetLoadAction OutputLoadAction = ERenderTargetLoadAction::ELoad;

	const FSlateClippingOp* ClippingOp = nullptr;
	const FDepthStencilBinding* ClippingStencilBinding = nullptr;
	FIntRect ClippingElementsViewRect;

	FIntRect OutputRect;
	FVector4f CornerRadius = FVector4f::Zero();
};

void AddSlatePostProcessUpsamplePass(FRDGBuilder& GraphBuilder, const FSlatePostProcessUpsampleInputs& Inputs)
{
	FSlatePostProcessUpsamplePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSlatePostProcessUpsamplePS::FParameters>();
	PassParameters->RenderTargets[0] = FRenderTargetBinding(Inputs.OutputTexture, Inputs.OutputLoadAction);

	if (Inputs.ClippingStencilBinding)
	{
		PassParameters->RenderTargets.DepthStencil = *Inputs.ClippingStencilBinding;
	}
	
	ESlatePostProcessUpsampleOutputFormat OutputFormat = ESlatePostProcessUpsampleOutputFormat::SDR;

	if (Inputs.OutputTextureToClear)
	{
		OutputFormat = Inputs.OutputTexture->Desc.Format == PF_FloatRGBA
			? ESlatePostProcessUpsampleOutputFormat::HDR_SCRGB
			: ESlatePostProcessUpsampleOutputFormat::HDR_PQ10;

		PassParameters->RenderTargets[1] = FRenderTargetBinding(Inputs.OutputTextureToClear, ERenderTargetLoadAction::ELoad);
	}

	FSlatePostProcessUpsamplePS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FSlatePostProcessUpsamplePS::FUpsampleOutputFormat>(OutputFormat);

	const ERHIFeatureLevel::Type FeatureLevel = GMaxRHIFeatureLevel;
	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(FeatureLevel);

	TShaderMapRef<FScreenPassVS> VertexShader(ShaderMap);
	TShaderMapRef<FSlatePostProcessUpsamplePS> PixelShader(ShaderMap, PermutationVector);

	const FScreenPassTextureViewport InputViewport(Inputs.InputTexture);
	const FScreenPassTextureViewport OutputViewport(Inputs.OutputTexture, Inputs.OutputRect);
	const FScreenPassTextureViewportParameters InputParameters = GetScreenPassTextureViewportParameters(InputViewport);

	PassParameters->ElementTexture = Inputs.InputTexture.Texture;
	PassParameters->ElementTextureSampler = Inputs.InputTexture.ViewRect == Inputs.OutputRect
		? TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI()
		: TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	PassParameters->ShaderParams = FVector4f(InputParameters.ViewportSize, InputParameters.UVViewportSize);
	PassParameters->ShaderParams2 = Inputs.CornerRadius;

	// RT0: overwrite when no rounded corners (skip dst read), alpha blend otherwise
	// RT1: punch out UI render target by (1 - src.a)
	FRHIBlendState* const BlendState = (Inputs.CornerRadius == FVector4f::Zero())
		? TStaticBlendState<
			CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			CW_RGBA, BO_Add, BF_Zero, BF_Zero, BO_Add, BF_Zero, BF_Zero>::GetRHI()
		: TStaticBlendState<
			CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_One, BF_InverseSourceAlpha,
			CW_RGBA, BO_Add, BF_Zero, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha>::GetRHI();

	FScreenPassPipelineState PipelineState(VertexShader, PixelShader, BlendState);
	GetSlateClippingPipelineState(Inputs.ClippingOp, PipelineState.DepthStencilState, PipelineState.StencilRef);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("Upsample"),
		PassParameters,
		ERDGPassFlags::Raster,
		[OutputViewport, InputViewport, ClippingElementsViewRect = Inputs.ClippingElementsViewRect, PipelineState, PixelShader, ClippingOp = Inputs.ClippingOp, PassParameters](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		if (ClippingOp && ClippingOp->Method == EClippingMethod::Stencil)
		{
			// Stencil clipping quads have their own viewport.
			RHICmdList.SetViewport(ClippingElementsViewRect.Min.X, ClippingElementsViewRect.Min.Y, 0.0f, ClippingElementsViewRect.Max.X, ClippingElementsViewRect.Max.Y, 1.0f);

			// Stencil clipping will issue its own draw calls.
			SetSlateClipping(RHICmdList, ClippingOp, ClippingElementsViewRect);
		}

		RHICmdList.SetViewport(OutputViewport.Rect.Min.X, OutputViewport.Rect.Min.Y, 0.0f, OutputViewport.Rect.Max.X, OutputViewport.Rect.Max.Y, 1.0f);

		if (ClippingOp && ClippingOp->Method == EClippingMethod::Scissor)
		{
			SetSlateClipping(RHICmdList, ClippingOp, ClippingElementsViewRect);
		}

		SetScreenPassPipelineState(RHICmdList, PipelineState);
		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);
		DrawScreenPass_PostSetup(RHICmdList, FScreenPassViewInfo(), OutputViewport, InputViewport, PipelineState, EScreenPassDrawFlags::None);
	});
}

//////////////////////////////////////////////////////////////////////////

class FSlatePostProcessOptimizedKawaseUpsamplePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FSlatePostProcessOptimizedKawaseUpsamplePS);
	SHADER_USE_PARAMETER_STRUCT(FSlatePostProcessOptimizedKawaseUpsamplePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ElementTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, ElementTextureSampler)
		SHADER_PARAMETER(FVector4f, ShaderParams)
		SHADER_PARAMETER(FVector4f, ShaderParams2)
		SHADER_PARAMETER(FVector4f, UVBounds)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FSlatePostProcessOptimizedKawaseUpsamplePS, "/Engine/Private/SlatePostProcessPixelShader.usf", "OptimizedKawaseUpsampleMain", SF_Pixel);

struct FSlateKawaseBlurUpsampleParameters
{
	float NearSampleWeight;
	float FarSampleWeight;
	float NearSampleOffset;
	float FarSampleOffset;
	FVector2f SideSampleOffsets;
};

struct FSlateKawaseBlurInternalConfiguration
{
	// The number of downsample & upsample stages.
	uint32 NumDownsampleLevels;
	// The number of same-size Kawase passes at the lowest downsample level.
	uint32 LowestLevelSteps;
	// The sample offset (in source pixels) for downsample passes. Also used for the first same-size pass if NumDownsampleLevels == 0 so that it can be different from the second pass.
	float DownsampleOffset;
	// The sample offset for same-size Kawase passes at the lowest downsample level.
	float ResampleOffset;
	// Parameters for upsample passes.
	FSlateKawaseBlurUpsampleParameters UpsampleParameters;
};

FSlateKawaseBlurUpsampleParameters GetSlateKawaseBlurUpsampleParameters(float Sigma)
{
	if (Sigma < 1.f) // Upsample pass is only done when Sigma > 1.25
	{
		return FSlateKawaseBlurUpsampleParameters();
	}
	// Function approximation from optimized datapoints - all coefficients algorithmically generated
	const float X = Sigma;
	const float X2 = X*X;
	FSlateKawaseBlurUpsampleParameters Parameters;
	if (Sigma >= 1.543f)
	{
		Parameters.NearSampleWeight = 0.258865f + 0.804972f * FMath::Exp(-0.035687f*X2 - 0.949113f*X + 0.065214f);
		Parameters.FarSampleWeight = 0.234567f - 5.067121f * FMath::Exp(-0.194387f*X2 - 0.205952f*X - 2.948161f);
		Parameters.NearSampleOffset = 0.908440f + 4.887250f/(-3.585551f*X2 + 3.686305f*X - 6.595607f);
		Parameters.SideSampleOffsets.X = 0.901689f + 1.158662f/(-1.046629f*X2 + 1.466162f*X - 2.044516f);
	}
	else
	{
		Parameters.NearSampleWeight = 0.975216f - 0.713882f * FMath::Sqrt(-0.250094f*X2 + 1.061462f*X - 0.483218f);
		Parameters.FarSampleWeight = -0.049043f*X2 + 0.222989f*X - 0.114601f;
		Parameters.NearSampleOffset = X > 1.25f ? FMath::Max(0.171405f + 0.268551f * FMath::Sqrt(3.329612f*X2 - 7.222064f*X + 3.883447f), 0.25f) : 0.25f;
		Parameters.SideSampleOffsets.X = FMath::Max(0.957531f - 2.539860f/(2.309942f*X2 - 2.981188f*X + 3.588718f), 0.25f);
	}
	Parameters.SideSampleOffsets.Y = FMath::Min(-1.420319f - 1.375146f/(-0.831627f*X2 + 1.249141f*X - 1.997956f), -0.75f);
	Parameters.FarSampleOffset = FMath::Min(-1.408462f - 1.766000f/(-1.348784f*X2 + 2.441929f*X - 3.247581f), -0.75f);
	return Parameters;
}

static FVector2f GetSlateKawaseTwoFullResolutionPassesOffsets(float Sigma) {
	// Function approximation from optimized datapoints - all coefficients algorithmically generated
	const float X = Sigma;
	const float X2 = X*X;
	const float X3 = X*X2;
	const float SqrtX = FMath::Sqrt(X);
	FVector2f Offsets(0.f);
	if (X >= .9125f && X <= 1.18f) // Middle section
	{
		Offsets = FVector2f(.611335f - 1.199396f * FMath::Exp(1.f / (2.318341f*X3 - 5.224627f*X2 + 2.320031f*X + 4.521828f*SqrtX - 4.418420f)));
	}
	else if (X < 1.f)
	{
		if (X < .8f)
		{
			if (X >= .25f)
			{
				Offsets[0] = 2.581554f * FMath::Exp(1.f / (-.581899f*X2 - .030902f*X));
			}
			if (X >= .2f)
			{
				Offsets[1] = FMath::Max(-.005631f + .527485f * FMath::Exp(1.f / (-8.671486f*X3 - 6.569889f*X2 + 9.311589f*X - 4.983829f*SqrtX + .479089f)), 1.f);
			}
		}
		else
		{
			Offsets[0] = 68.634628f*X3 - 162.504166f*X2 + 103.115616f*X + 47.546867f*SqrtX - 55.950523f;
			Offsets[1] = .433818f - 7.626031f * FMath::Exp(1.f - 1.f / (6.921899f*X3 - .834526f*X2 - 15.626254f*X + 3.757118f*SqrtX + 6.258511f));
		}
	}
	else
	{
		Offsets[0] = FMath::Min(.535894f - .009497f * FMath::Loge(FMath::Max(-3.618996f*X3 + 6.220135f*X2 + 3.839638f*X - 7.252954f, .000001f)), .594476f);
		Offsets[1] = 1.263563f + .098106f * FMath::Loge(1.174112f*X3 - 4.153324f*X2 + 4.932609f*X - 1.965301f);
	}
	return Offsets;
}

FSlateKawaseBlurInternalConfiguration GetSlateKawaseBlurInternalConfiguration(float Sigma)
{
	// Constants and coefficients found by fitting optimized datapoints
	constexpr float SIGMA_TO_LEVELS_LOG_BASE = 2.043f; // The number of downscale & upscale stages increases by one every time Sigma is multiplied by this value

	FSlateKawaseBlurInternalConfiguration Configuration = { };
	if (Sigma <= .27f) // No blur
	{
		return Configuration;
	}
	if (Sigma <= .8f) // Single pass at full size
	{
		const float X = Sigma;
		Configuration.LowestLevelSteps = 1;
		Configuration.DownsampleOffset = Configuration.ResampleOffset = 0.255305f * FMath::Exp(1.f - 1.f / (2.392349f*X*X*X + 3.736583f*X*X - 1.020735f*X + 0.123777f));
	}
	else if (Sigma < 4.f/3.f) // Two passes at full size
	{
		const FVector2f Offsets = GetSlateKawaseTwoFullResolutionPassesOffsets(Sigma);
		Configuration.LowestLevelSteps = 2;
		Configuration.DownsampleOffset = Offsets[0];
		Configuration.ResampleOffset = Offsets[1];
	}
	else
	{
		Configuration.NumDownsampleLevels = FMath::Max(int32((FMath::Loge(Sigma) - FMath::Loge(3.5f)) / FMath::Loge(SIGMA_TO_LEVELS_LOG_BASE) + 2.f), 1);
		Configuration.DownsampleOffset = 7.f/9.f; // This has been found to be very close to the optimal value for all remaining cases.

		const float StageSigma = Configuration.NumDownsampleLevels <= 1 ? Sigma : Sigma * FMath::Pow(SIGMA_TO_LEVELS_LOG_BASE, 1.f - float(Configuration.NumDownsampleLevels)) - 0.16f;
		float UpsampleSigma = StageSigma;
		if (StageSigma > 2.f) {
			UpsampleSigma = 2.f;
			Configuration.LowestLevelSteps = 1;
			float X = StageSigma;
			if (Configuration.NumDownsampleLevels > 1)
			{
				X = 1.16f * (X - SIGMA_TO_LEVELS_LOG_BASE) + SIGMA_TO_LEVELS_LOG_BASE;
			}
			Configuration.ResampleOffset = 1.664566f - FMath::Exp(-0.082114f*X*X - 0.387889f*X + 1.595579f);
			if (Configuration.ResampleOffset > Configuration.DownsampleOffset) {
				Configuration.LowestLevelSteps = 2;
				Configuration.ResampleOffset = Configuration.NumDownsampleLevels <= 1 ? .625f * (StageSigma - SIGMA_TO_LEVELS_LOG_BASE) : 2.f/3.f * (StageSigma - 2.f);
			}
		}
		Configuration.UpsampleParameters = GetSlateKawaseBlurUpsampleParameters(UpsampleSigma);
	}
	return Configuration;
}

static FScreenPassTexture AddSlateKawaseBlurSymmetricalPass(FRDGBuilder& GraphBuilder, const FScreenPassTexture& InputTexture, uint32 DownscaleFactor, float SampleOffset)
{
	const ERHIFeatureLevel::Type FeatureLevel = GMaxRHIFeatureLevel;
	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(FeatureLevel);

	const FIntPoint OutputExtent = DownscaleFactor > 1 ? GetDownscaledExtent(InputTexture.ViewRect.Size(), DownscaleFactor) : InputTexture.ViewRect.Size();
	TShaderMapRef<FSlatePostProcessResample2x2PS> PixelShader(ShaderMap);
	const FScreenPassRenderTarget Output(
		GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(OutputExtent, InputTexture.Texture->Desc.Format, FClearValueBinding::None, GetSlateTransientRenderTargetFlags()), TEXT("PostBlurUI")),
		ERenderTargetLoadAction::ENoAction);
	
	const FScreenPassTextureViewport InputViewport(InputTexture);

	// This ensures that if input resolution is not divisible by the DownscaleFactor, we actually scale exactly DownscaleFactor x DownscaleFactor pixels into one and pad the last row and column rather than just fitting to the rounded dimensions.
	const FScreenPassTextureViewport ProportionalInputViewport(InputTexture.Texture, FIntRect(InputTexture.ViewRect.Min, InputTexture.ViewRect.Min + FIntPoint(DownscaleFactor) * OutputExtent));
	// Input parameters are still computed from actual InputViewport, otherwise pixels from outside the viewport get blended into the last row and column.
	const FScreenPassTextureViewportParameters InputParameters = GetScreenPassTextureViewportParameters(InputViewport);
	const FScreenPassTextureViewport OutputViewport(Output);

	FSlatePostProcessResample2x2PS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSlatePostProcessResample2x2PS::FParameters>();
	PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
	PassParameters->ElementTexture = InputTexture.Texture;
	PassParameters->ElementTextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
	PassParameters->ShaderParams = SampleOffset * FVector4f(InputParameters.ExtentInverse.X, InputParameters.ExtentInverse.Y, 0.0f, 0.0f);
	PassParameters->UVBounds = FVector4f(InputParameters.UVViewportBilinearMin, InputParameters.UVViewportBilinearMax);

	AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("PostBlurUI"), GMaxRHIFeatureLevel, OutputViewport, ProportionalInputViewport, PixelShader, PassParameters);
	return Output;
}

static FScreenPassTexture AddSlateKawaseBlurUpsamplePass(FRDGBuilder& GraphBuilder, const FScreenPassTexture& InputTexture, const FIntPoint& OutputExtent, const FSlateKawaseBlurUpsampleParameters& Parameters)
{
	const ERHIFeatureLevel::Type FeatureLevel = GMaxRHIFeatureLevel;
	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(FeatureLevel);

	TShaderMapRef<FSlatePostProcessOptimizedKawaseUpsamplePS> PixelShader(ShaderMap);
	const FScreenPassRenderTarget Output(
		GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(OutputExtent, InputTexture.Texture->Desc.Format, FClearValueBinding::None, GetSlateTransientRenderTargetFlags()), TEXT("PostBlurUI")),
		ERenderTargetLoadAction::ENoAction);
	
	const FScreenPassTextureViewport InputViewport(InputTexture);
	const FScreenPassTextureViewportParameters InputParameters = GetScreenPassTextureViewportParameters(InputViewport);
	const FScreenPassTextureViewport OutputViewport(Output.Texture, FIntRect(FIntPoint(0), FIntPoint(2) * InputTexture.ViewRect.Size()));

	FSlatePostProcessOptimizedKawaseUpsamplePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSlatePostProcessOptimizedKawaseUpsamplePS::FParameters>();
	PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
	PassParameters->ElementTexture = InputTexture.Texture;
	PassParameters->ElementTextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
	PassParameters->ShaderParams = FVector4f(InputParameters.ExtentInverse.X, InputParameters.ExtentInverse.Y, Parameters.NearSampleWeight, Parameters.FarSampleWeight);
	PassParameters->ShaderParams2 = FVector4f(Parameters.SideSampleOffsets.X, Parameters.SideSampleOffsets.Y, Parameters.NearSampleOffset, Parameters.FarSampleOffset);
	PassParameters->UVBounds = FVector4f(InputParameters.UVViewportBilinearMin, InputParameters.UVViewportBilinearMax);

	AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("PostBlurUI"), GMaxRHIFeatureLevel, OutputViewport, InputViewport, PixelShader, PassParameters);
	return Output;
}

FScreenPassTexture AddSlateKawaseBlur(FRDGBuilder& GraphBuilder, const FScreenPassTexture& InputTexture, const FSlateKawaseBlurInternalConfiguration& Configuration)
{
	if (Configuration.NumDownsampleLevels == 0 && Configuration.LowestLevelSteps == 0)
	{
		return InputTexture;
	}

	FScreenPassTexture BlurTexture(InputTexture);
	TArray<FIntPoint, TInlineAllocator<256>> UpsampleStageExtents;
	UpsampleStageExtents.SetNumUninitialized(Configuration.NumDownsampleLevels);
	// Downsample passes
	for (uint32 DownsampleStage = 0; DownsampleStage < Configuration.NumDownsampleLevels; ++DownsampleStage)
	{
		UpsampleStageExtents[DownsampleStage] = BlurTexture.ViewRect.Size();
		BlurTexture = AddSlateKawaseBlurSymmetricalPass(GraphBuilder, BlurTexture, 2, Configuration.DownsampleOffset);
	}
	// Lowest level same-size passes
	for (uint32 LowestLevelStep = 0; LowestLevelStep < Configuration.LowestLevelSteps; ++LowestLevelStep)
	{
		const float SampleOffset = Configuration.NumDownsampleLevels == 0 && LowestLevelStep == 0 ? Configuration.DownsampleOffset : Configuration.ResampleOffset;
		BlurTexture = AddSlateKawaseBlurSymmetricalPass(GraphBuilder, BlurTexture, 1, SampleOffset);
	}
	// Upsample passes
	for (int32 UpsampleStage = int32(Configuration.NumDownsampleLevels)-1; UpsampleStage >= 0; --UpsampleStage)
	{
		BlurTexture = AddSlateKawaseBlurUpsamplePass(GraphBuilder, BlurTexture, UpsampleStageExtents[UpsampleStage], Configuration.UpsampleParameters);
	}
	return BlurTexture;
}

//////////////////////////////////////////////////////////////////////////

class FSlatePostProcessBlurPS : public FGlobalShader
{
public:
	static const int32 MAX_BLUR_SAMPLES = 127 / 2;

	DECLARE_GLOBAL_SHADER(FSlatePostProcessBlurPS);
	SHADER_USE_PARAMETER_STRUCT(FSlatePostProcessBlurPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ElementTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, ElementTextureSampler)
		SHADER_PARAMETER_ARRAY(FVector4f, WeightAndOffsets, [MAX_BLUR_SAMPLES])
		SHADER_PARAMETER(uint32, SampleCount)
		SHADER_PARAMETER(FVector4f, BufferSizeAndDirection)
		SHADER_PARAMETER(FVector4f, UVBounds)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FSlatePostProcessBlurPS, "/Engine/Private/SlatePostProcessPixelShader.usf", "GaussianBlurMain", SF_Pixel);

FScreenPassTexture AddSlatePostProcessOldGaussianBlur(FRDGBuilder& GraphBuilder, const FSlatePostProcessBlurPassInputs& Inputs)
{
	const auto GetWeight = [](float Dist, float Strength)
	{
		float Strength2 = Strength*Strength;
		return (1.0f / FMath::Sqrt(2 * PI*Strength2))*FMath::Exp(-(Dist*Dist) / (2 * Strength2));
	};

	const auto GetWeightsAndOffset = [GetWeight](float Dist, float Sigma)
	{
		float Offset1 = Dist;
		float Weight1 = GetWeight(Offset1, Sigma);

		float Offset2 = Dist + 1;
		float Weight2 = GetWeight(Offset2, Sigma);

		float TotalWeight = Weight1 + Weight2;
		float Offset = 0;

		if (TotalWeight > 0)
		{
			Offset = (Weight1 * Offset1 + Weight2 * Offset2) / TotalWeight;
		}

		return FVector2f(TotalWeight, Offset);
	};

	const int32 SampleCount = FMath::DivideAndRoundUp(Inputs.KernelSize, 2u);

	// We need half of the sample count array because we're packing two samples into one float;
	TArray<FVector4f, FRDGArrayAllocator> WeightsAndOffsets;
	WeightsAndOffsets.AddUninitialized(SampleCount % 2 == 0 ? SampleCount / 2 : SampleCount / 2 + 1);
	WeightsAndOffsets[0] = FVector4f(FVector2f(GetWeight(0, Inputs.Strength), 0), GetWeightsAndOffset(1, Inputs.Strength) );

	for (uint32 X = 3, SampleIndex = 1; X < Inputs.KernelSize; X += 4, SampleIndex++)
	{
		WeightsAndOffsets[SampleIndex] = FVector4f(GetWeightsAndOffset((float)X, Inputs.Strength), GetWeightsAndOffset((float)(X + 2), Inputs.Strength));
	}

	FScreenPassTextureViewport OutputTextureViewport(Inputs.InputRect.Size());

	const EPixelFormat InputPixelFormat = Inputs.InputTexture->Desc.Format;

	// Defaults to the input UI texture unless a downsample / composite pass is needed.
	FScreenPassTexture BlurInputTexture(Inputs.InputTexture, Inputs.InputRect);

	const ERHIFeatureLevel::Type FeatureLevel = GMaxRHIFeatureLevel;
	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(FeatureLevel);
	TShaderMapRef<FSlatePostProcessBlurPS> PixelShader(ShaderMap);

	FScreenPassRenderTarget BlurOutputTexture(
		GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(OutputTextureViewport.Extent, InputPixelFormat, FClearValueBinding::None, GetSlateTransientRenderTargetFlags()), TEXT("SlateBlurHorizontalTexture")),
		ERenderTargetLoadAction::ENoAction);

	FSlatePostProcessBlurPS::FParameters* PassParameters = nullptr;

	{
		const FScreenPassTextureViewport BlurInputViewport(BlurInputTexture);
		const FScreenPassTextureViewportParameters BlurInputParameters = GetScreenPassTextureViewportParameters(BlurInputViewport);

		PassParameters = GraphBuilder.AllocParameters<FSlatePostProcessBlurPS::FParameters>();
		PassParameters->RenderTargets[0] = BlurOutputTexture.GetRenderTargetBinding();
		PassParameters->ElementTexture = BlurInputTexture.Texture;
		PassParameters->ElementTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->SampleCount = SampleCount;
		PassParameters->BufferSizeAndDirection = FVector4f(BlurInputParameters.ExtentInverse, FVector2f(1.0f, 0.0f));
		PassParameters->UVBounds = FVector4f(BlurInputParameters.UVViewportBilinearMin, BlurInputParameters.UVViewportBilinearMax);

		check(PassParameters->WeightAndOffsets.Num() * sizeof(PassParameters->WeightAndOffsets[0]) >= WeightsAndOffsets.Num() * sizeof(WeightsAndOffsets[0]));
		FPlatformMemory::Memcpy(PassParameters->WeightAndOffsets.GetData(), WeightsAndOffsets.GetData(), WeightsAndOffsets.Num() * sizeof(WeightsAndOffsets[0]));

		AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("Horizontal"), FeatureLevel, FScreenPassTextureViewport(BlurOutputTexture), BlurInputViewport, PixelShader, PassParameters);
	}

	BlurInputTexture = BlurOutputTexture;
	BlurOutputTexture = FScreenPassRenderTarget(
		GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(OutputTextureViewport.Extent, InputPixelFormat, FClearValueBinding::None, GetSlateTransientRenderTargetFlags()), TEXT("SlateBlurVerticalTexture")),
		ERenderTargetLoadAction::ENoAction);

	{
		const FScreenPassTextureViewport BlurInputViewport(BlurInputTexture);
		const FScreenPassTextureViewportParameters BlurInputParameters = GetScreenPassTextureViewportParameters(BlurInputViewport);

		PassParameters = GraphBuilder.AllocParameters<FSlatePostProcessBlurPS::FParameters>();
		PassParameters->RenderTargets[0] = BlurOutputTexture.GetRenderTargetBinding();
		PassParameters->ElementTexture = BlurInputTexture.Texture;
		PassParameters->ElementTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->SampleCount = SampleCount;
		PassParameters->BufferSizeAndDirection = FVector4f(BlurInputParameters.ExtentInverse, FVector2f(0.0f, 1.0f));
		PassParameters->UVBounds = FVector4f(BlurInputParameters.UVViewportBilinearMin, BlurInputParameters.UVViewportBilinearMax);

		check(PassParameters->WeightAndOffsets.Num() * sizeof(PassParameters->WeightAndOffsets[0]) >= WeightsAndOffsets.Num() * sizeof(WeightsAndOffsets[0]));
		FPlatformMemory::Memcpy(PassParameters->WeightAndOffsets.GetData(), WeightsAndOffsets.GetData(), WeightsAndOffsets.Num() * sizeof(WeightsAndOffsets[0]));

		AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("Vertical"), FeatureLevel, FScreenPassTextureViewport(BlurOutputTexture), BlurInputViewport, PixelShader, PassParameters);
	}

	return BlurOutputTexture;
}

void AddSlatePostProcessBlurPass(FRDGBuilder& GraphBuilder, const FSlatePostProcessBlurPassInputs& Inputs)
{
	RDG_EVENT_SCOPE(GraphBuilder, "GaussianBlur");
	CSV_CUSTOM_STAT(Slate, PostProcessBlurPassCount, 1, ECsvCustomStatOp::Accumulate);

	FIntRect UnscaledRect(Inputs.OutputRect);
	FScreenPassTexture BlurInputTexture(Inputs.InputTexture, Inputs.InputRect);
	FScreenPassTexture BlurOutputTexture;
	FScreenPassTextureViewport OutputTextureViewport(Inputs.InputRect.Size());
	if (Inputs.DownsampleAmount > 1)
	{
		OutputTextureViewport = FScreenPassTextureViewport(GetDownscaledExtent(Inputs.InputRect.Size(), Inputs.DownsampleAmount));
	}

	// Need to composite the HDR scene texture with a separate SDR UI texture (which also does a downsample).
	if (Inputs.SDRCompositeUITexture)
	{
		FSlateCompositeHDRForBlurPassInputs CompositeInputs;
		CompositeInputs.InputRect = Inputs.InputRect;
		CompositeInputs.InputTexture = Inputs.InputTexture;
		CompositeInputs.InputCompositeTexture = Inputs.SDRCompositeUITexture;
		CompositeInputs.OutputExtent = OutputTextureViewport.Extent;
		CompositeInputs.HDRPaperWhiteInNits = Inputs.HDRPaperWhiteInNits;

		BlurInputTexture = AddSlateCompositeHDRForBlurPass(GraphBuilder, CompositeInputs);
	}
	// Need to do an explicit downsample pass.
	else if (Inputs.DownsampleAmount > 1)
	{
		FSlatePostProcessDownsamplePassInputs DownsampleInputs;
		DownsampleInputs.InputTexture = BlurInputTexture;
		DownsampleInputs.OutputExtent = OutputTextureViewport.Extent;
		DownsampleInputs.Downscale = Inputs.DownsampleAmount;

		BlurInputTexture = AddSlatePostProcessDownsamplePass(GraphBuilder, DownsampleInputs);

		// If InputRect dimensions are not divisible by DownsampleAmount, this fixes up the subpixel alignment, since AddSlatePostProcessDownsamplePass rounds up the input extent.
		// For example if input width is 9 and downscale is 2x, the downsample pass actually takes 10 and shrinks to 5. Therefore, we need to upscale back to 10 and not 9 pixels.
		const FIntPoint UnscaledSize = FIntPoint(Inputs.DownsampleAmount) * BlurInputTexture.ViewRect.Size();
		if (Inputs.InputRect.Size() == Inputs.OutputRect.Size())
		{
			UnscaledRect.Max = UnscaledRect.Min + UnscaledSize;
		}
		else if (Inputs.InputRect.Width() != 0 && Inputs.InputRect.Height() != 0)
		{
			UnscaledRect.Max = UnscaledRect.Min + (FVector2f(Inputs.OutputRect.Size()) / FVector2f(Inputs.InputRect.Size()) * FVector2f(UnscaledSize)).IntPoint();
		}
	}

	if (SlatePostBlurDualKawaseFilterEnable)
	{
		const FSlateKawaseBlurInternalConfiguration Configuration = GetSlateKawaseBlurInternalConfiguration(Inputs.Strength);
		BlurOutputTexture = AddSlateKawaseBlur(GraphBuilder, BlurInputTexture, Configuration);
	}
	else
	{
		FSlatePostProcessBlurPassInputs DownscaledInputs(Inputs);
		DownscaledInputs.InputTexture = BlurInputTexture.Texture;
		DownscaledInputs.InputRect = BlurInputTexture.ViewRect;
		DownscaledInputs.DownsampleAmount = 0;
		BlurOutputTexture = AddSlatePostProcessOldGaussianBlur(GraphBuilder, DownscaledInputs);
	}

	FSlatePostProcessUpsampleInputs UpsampleInputs;
	UpsampleInputs.InputTexture = BlurOutputTexture;
	UpsampleInputs.OutputTextureToClear = Inputs.SDRCompositeUITexture;
	UpsampleInputs.OutputTexture = Inputs.OutputTexture;
	UpsampleInputs.OutputRect = UnscaledRect;
	UpsampleInputs.ClippingOp = Inputs.ClippingOp;
	UpsampleInputs.ClippingStencilBinding = Inputs.ClippingStencilBinding;
	UpsampleInputs.ClippingElementsViewRect = Inputs.ClippingElementsViewRect;
	UpsampleInputs.CornerRadius = Inputs.CornerRadius;

	AddSlatePostProcessUpsamplePass(GraphBuilder, UpsampleInputs);
}

void AddSlatePostProcessCopy(FRDGBuilder& GraphBuilder, FScreenPassTexture Input, FScreenPassTexture Output)
{
	if (Input.ViewRect.Size() == Output.ViewRect.Size())
	{
		AddDrawTexturePass(GraphBuilder, FScreenPassViewInfo(), Input, Output);
	}
	else
	{
		// Like AddDrawTexturePass but with SF_Bilinear

		const FScreenPassRenderTarget OutputTarget(Output, ERenderTargetLoadAction::ELoad);
		const FScreenPassTextureViewport InputViewport(Input);
		const FScreenPassTextureViewport OutputViewport(Output);

		TShaderMapRef<FCopyRectPS> PixelShader(GetGlobalShaderMap(FScreenPassViewInfo().FeatureLevel));

		FCopyRectPS::FParameters* Parameters = GraphBuilder.AllocParameters<FCopyRectPS::FParameters>();
		Parameters->InputTexture = Input.Texture;
		Parameters->InputSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		Parameters->RenderTargets[0] = OutputTarget.GetRenderTargetBinding();
		Parameters->RenderTargets.MultiViewCount = 1;

		AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("DrawTexture"), FScreenPassViewInfo(), OutputViewport, InputViewport, PixelShader, Parameters);
	}
}

void AddSlatePostProcessBlurPass(FRDGBuilder& GraphBuilder, const FSlatePostProcessSimpleBlurPassInputs& SimpleInputs)
{
	const int32 MinKernelSize = 3;
	const int32 MaxKernelSize = 255;
	const int32 Downsample2Threshold = 9; // Reached at blur strength 3.166
	const int32 Downsample3Threshold = 64; // Reached at blur strength 21.166
	const int32 Downsample4Threshold = 96; // Reached at blur strength 31.833
	const float StrengthToKernelSize = 3.0f;
	const float MinStrength = 0.5f;

	float Strength = FMath::Max(MinStrength, SimpleInputs.Strength);
	if (SlatePostBlurStrengthOverride > 0.f)
	{
		Strength = FMath::Max(MinStrength, SlatePostBlurStrengthOverride);
	}
	int32 KernelSize = FMath::RoundToInt(Strength * StrengthToKernelSize);
	int32 DownsampleAmount = 0;

	if (KernelSize > Downsample2Threshold)
	{
		DownsampleAmount = KernelSize >= Downsample3Threshold ? KernelSize >= Downsample4Threshold ? 4 : 3 : 2;
		KernelSize /= DownsampleAmount;
	}

	// Kernel sizes must be odd
	if (KernelSize % 2 == 0)
	{
		++KernelSize;
	}

	if (DownsampleAmount > 0)
	{
		Strength /= DownsampleAmount;
	}

	KernelSize = FMath::Clamp(KernelSize, MinKernelSize, MaxKernelSize);

	FSlatePostProcessBlurPassInputs Inputs;
	Inputs.InputTexture     = SimpleInputs.InputTexture.Texture;
	Inputs.InputRect        = SimpleInputs.InputTexture.ViewRect;
	Inputs.OutputTexture    = SimpleInputs.OutputTexture.Texture;
	Inputs.OutputRect       = SimpleInputs.OutputTexture.ViewRect;
	Inputs.KernelSize       = KernelSize;
	Inputs.Strength         = Strength;
	Inputs.DownsampleAmount = DownsampleAmount;

	AddSlatePostProcessBlurPass(GraphBuilder, Inputs);
}

//////////////////////////////////////////////////////////////////////////

class FSlatePostProcessColorDeficiencyPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FSlatePostProcessColorDeficiencyPS);
	SHADER_USE_PARAMETER_STRUCT(FSlatePostProcessColorDeficiencyPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ElementTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, ElementTextureSampler)
		SHADER_PARAMETER(float, ColorVisionDeficiencyType)
		SHADER_PARAMETER(float, ColorVisionDeficiencySeverity)
		SHADER_PARAMETER(float, bCorrectDeficiency)
		SHADER_PARAMETER(float, bSimulateCorrectionWithDeficiency)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FSlatePostProcessColorDeficiencyPS, "/Engine/Private/SlatePostProcessColorDeficiencyPixelShader.usf", "ColorDeficiencyMain", SF_Pixel);

void AddSlatePostProcessColorDeficiencyPass(FRDGBuilder& GraphBuilder, const FSlatePostProcessColorDeficiencyPassInputs& Inputs)
{
	const ERHIFeatureLevel::Type FeatureLevel = GMaxRHIFeatureLevel;
	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(FeatureLevel);
	TShaderMapRef<FSlatePostProcessColorDeficiencyPS> PixelShader(ShaderMap);
	const FRDGTextureDesc& InputDesc = Inputs.InputTexture.Texture->Desc;

	const FScreenPassRenderTarget Output(
		GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(InputDesc.Extent, InputDesc.Format, FClearValueBinding::None, GetSlateTransientRenderTargetFlags()), TEXT("ColorDeficiency")),
		ERenderTargetLoadAction::ENoAction);

	FSlatePostProcessColorDeficiencyPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSlatePostProcessColorDeficiencyPS::FParameters>();
	PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
	PassParameters->ElementTexture = Inputs.InputTexture.Texture;
	PassParameters->ElementTextureSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters->ColorVisionDeficiencyType = (float)GSlateColorDeficiencyType;
	PassParameters->ColorVisionDeficiencySeverity = (float)GSlateColorDeficiencySeverity;
	PassParameters->bCorrectDeficiency = GSlateColorDeficiencyCorrection ? 1.0f : 0.0f;
	PassParameters->bSimulateCorrectionWithDeficiency = GSlateShowColorDeficiencyCorrectionWithDeficiency ? 1.0f : 0.0f;

	const FScreenPassTextureViewport Viewport(Output);
	AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("ColorDeficiency"), FeatureLevel, Viewport, Viewport, PixelShader, PassParameters);

	FSlatePostProcessUpsampleInputs UpsampleInputs;
	UpsampleInputs.InputTexture = Output;
	UpsampleInputs.OutputTexture = Inputs.OutputTexture.Texture;
	UpsampleInputs.OutputRect = Inputs.OutputTexture.ViewRect;

	AddSlatePostProcessUpsamplePass(GraphBuilder, UpsampleInputs);
}
