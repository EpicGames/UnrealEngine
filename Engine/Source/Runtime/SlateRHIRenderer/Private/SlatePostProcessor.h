// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Layout/SlateRect.h"
#include "Layout/Clipping.h"
#include "ScreenPass.h"

struct FSlateClippingOp
{
	union
	{
		struct
		{
			FSlateRect Rect;
		} Data_Scissor;

		struct
		{
			TConstArrayView<FSlateClippingZone> Zones;
		} Data_Stencil;
	};

	FVector2f Offset;
	EClippingMethod Method;
	uint8 MaskingId;

	static inline FSlateClippingOp* Scissor(FRDGBuilder& GraphBuilder, FVector2f Offset, FSlateRect Rect)
	{
		FSlateClippingOp* Op = GraphBuilder.AllocPOD<FSlateClippingOp>();
		Op->Data_Scissor.Rect = Rect;
		Op->Offset = Offset;
		Op->Method = EClippingMethod::Scissor;
		Op->MaskingId = 0;
		return Op;
	}

	static inline FSlateClippingOp* Stencil(FRDGBuilder& GraphBuilder, FVector2f Offset, TConstArrayView<FSlateClippingZone> Zones, int32 MaskingId)
	{
		FSlateClippingOp* Op = GraphBuilder.AllocPOD<FSlateClippingOp>();
		Op->Data_Stencil.Zones = Zones;
		Op->Offset = Offset;
		Op->Method = EClippingMethod::Stencil;
		Op->MaskingId = MaskingId;
		return Op;
	}
};

bool GetSlateClippingPipelineState(const FSlateClippingOp* ClippingStateOp, FRHIDepthStencilState*& OutDepthStencilState, uint8& OutStencilRef);

void SetSlateClipping(FRHICommandList& RHICmdList, const FSlateClippingOp* ClippingStateOp, FIntRect ViewportRect);

struct FSlatePostProcessBlurPassInputs
{
	// An optional in/out separately composited UI texture that is composited with the input and then the output rect is reset to transparent.
	FRDGTexture* SDRCompositeUITexture = nullptr;
	FRDGTexture* InputTexture = nullptr;
	FRDGTexture* OutputTexture = nullptr;
	ERenderTargetLoadAction OutputLoadAction = ERenderTargetLoadAction::ELoad;

	// An optional set of inputs for when a blur is performed as part of a slate render batch.
	const FSlateClippingOp* ClippingOp = nullptr;
	const FDepthStencilBinding* ClippingStencilBinding = nullptr;
	FIntRect ClippingElementsViewRect;

	FIntRect InputRect;
	FIntRect OutputRect;
	uint32 KernelSize = 0;
	float  Strength = 0.0f;
	uint32 DownsampleAmount = 0;
	FVector4f CornerRadius = FVector4f::Zero();
	float HDRPaperWhiteInNits = 203.f;
};

extern ETextureCreateFlags GetSlateTransientRenderTargetFlags();
extern ETextureCreateFlags GetSlateTransientDepthStencilFlags();

/** Directly copies or resamples Input into Output with bilinear interpolation if the extent is different. */
SLATERHIRENDERER_API void AddSlatePostProcessCopy(FRDGBuilder& GraphBuilder, FScreenPassTexture Input, FScreenPassTexture Output);

SLATERHIRENDERER_API void AddSlatePostProcessBlurPass(FRDGBuilder& GraphBuilder, const FSlatePostProcessBlurPassInputs& Inputs);

struct FSlatePostProcessColorDeficiencyPassInputs
{
	FScreenPassTexture InputTexture;
	FScreenPassTexture OutputTexture;
};

void AddSlatePostProcessColorDeficiencyPass(FRDGBuilder& GraphBuilder, const FSlatePostProcessColorDeficiencyPassInputs& Inputs);
