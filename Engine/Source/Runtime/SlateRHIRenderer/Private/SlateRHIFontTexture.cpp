// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateRHIFontTexture.h"
#include "Rendering/SlateRenderer.h"
#include "RenderingThread.h"
#include "RenderUtils.h"

FSlateFontTextureRHIResource::FSlateFontTextureRHIResource(uint32 InWidth, uint32 InHeight, ESlateFontAtlasContentType InContentType)
	: Width(InWidth)
	, Height(InHeight)
	, ContentType(InContentType)
{
}

void FSlateFontTextureRHIResource::InitRHI(FRHICommandListBase& RHICmdList)
{
	check( IsInRenderingThread() );

	// Create the texture
	if( Width > 0 && Height > 0 )
	{
		const EPixelFormat PixelFormat = GetRHIPixelFormat();

		check( !IsValidRef( ShaderResource) );

		const static FLazyName ClassName(TEXT("FSlateFontTextureRHIResource"));
		FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("FSlateFontTextureRHIResource"), Width, Height, PixelFormat)
			.SetClassName(ClassName);

		switch (ContentType) {
			case ESlateFontAtlasContentType::Alpha:
			case ESlateFontAtlasContentType::Msdf:
				break;
			case ESlateFontAtlasContentType::Color:
				Desc.AddFlags(ETextureCreateFlags::SRGB);
				break;
			default:
				checkNoEntry();
				// Default to Color
				Desc.AddFlags(ETextureCreateFlags::SRGB);
				break;
		}

		ShaderResource = RHICmdList.CreateTexture(Desc);

		check( IsValidRef( ShaderResource ) );

		// Also assign the reference to the FTextureResource variable so that the Engine can access it
		TextureRHI = ShaderResource;

		// Create the sampler state RHI resource.
		FSamplerStateInitializerRHI SamplerStateInitializer
		(
		  SF_Bilinear,
		  AM_Clamp,
		  AM_Clamp,
		  AM_Wrap,
		  0,
		  1, // Disable anisotropic filtering, since aniso doesn't respect MaxLOD
		  0,
		  0
		);
		SamplerStateRHI = GetOrCreateSamplerState(SamplerStateInitializer);

		// Create a custom sampler state for using this texture in a deferred pass, where ddx / ddy are discontinuous
		FSamplerStateInitializerRHI DeferredPassSamplerStateInitializer
		(
		  SF_Bilinear,
		  AM_Clamp,
		  AM_Clamp,
		  AM_Wrap,
		  0,
		  1, // Disable anisotropic filtering, since aniso doesn't respect MaxLOD
		  0,
		  0
		);
		DeferredPassSamplerStateRHI = GetOrCreateSamplerState(DeferredPassSamplerStateInitializer);

		INC_MEMORY_STAT_BY(STAT_SlateTextureGPUMemory, Width*Height*GPixelFormats[PixelFormat].BlockBytes);
	}
}

void FSlateFontTextureRHIResource::ReleaseRHI()
{
	check( IsInRenderingThread() );

	// Release the texture
	if( IsValidRef(ShaderResource) )
	{
		const EPixelFormat PixelFormat = GetRHIPixelFormat();

		DEC_MEMORY_STAT_BY(STAT_SlateTextureGPUMemory, Width*Height*GPixelFormats[PixelFormat].BlockBytes);
	}

	ShaderResource.SafeRelease();
}

EPixelFormat FSlateFontTextureRHIResource::GetRHIPixelFormat() const
{
	switch (ContentType) {
		case ESlateFontAtlasContentType::Alpha:
			return PF_A8;
		case ESlateFontAtlasContentType::Color:
		case ESlateFontAtlasContentType::Msdf:
			return PF_B8G8R8A8;
		default:
			checkNoEntry();
			// Default to Color
			return PF_B8G8R8A8;
	}
}

FSlateFontAtlasRHI::FSlateFontAtlasRHI(uint32 Width, uint32 Height, ESlateFontAtlasContentType InContentType, ESlateTextureAtlasPaddingStyle InPaddingStyle)
	: FSlateFontAtlas(Width, Height, InContentType, InPaddingStyle)
	, FontTexture(new FSlateFontTextureRHIResource(Width, Height, InContentType))
{
	if (InContentType == ESlateFontAtlasContentType::Msdf)
	{
		// Actually this should be done for all content types but to be safe, I want to avoid affecting non-MSDF code for now.
		bNeedsUpdate = true;
	}
}

FSlateFontAtlasRHI::~FSlateFontAtlasRHI()
{
}

void FSlateFontAtlasRHI::ReleaseResources()
{
	checkSlow( IsThreadSafeForSlateRendering() );

	BeginReleaseResource( FontTexture.Get() );
}

void FSlateFontAtlasRHI::ConditionalUpdateTexture()
{
	if( bNeedsUpdate )
	{
		if (IsInRenderingThread())
		{
			FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
			FontTexture->InitResource(RHICmdList);

			uint32 DestStride;
			uint8* TempData = (uint8*)RHICmdList.LockTexture2D( FontTexture->GetTypedResource(), 0, RLM_WriteOnly, /*out*/ DestStride, false );
			// check( DestStride == Atlas.BytesPerPixel * Atlas.AtlasWidth ); // Temporarily disabling check
			FMemory::Memcpy( TempData, AtlasData.GetData(), GetSlateFontAtlasContentBytesPerPixel(ContentType)*AtlasWidth*AtlasHeight );
			RHICmdList.UnlockTexture2D( FontTexture->GetTypedResource(),0,false );
		}
		else
		{
			checkSlow( IsThreadSafeForSlateRendering() );

			BeginInitResource( FontTexture.Get() );

			FSlateFontAtlasRHI* Atlas = this;
			ENQUEUE_RENDER_COMMAND(SlateUpdateFontAtlasTextureCommand)(
				[Atlas](FRHICommandListImmediate& RHICmdList)
				{
					uint32 DestStride;
					uint8* TempData = (uint8*)RHICmdList.LockTexture2D( Atlas->FontTexture->GetTypedResource(), 0, RLM_WriteOnly, /*out*/ DestStride, false );
					// check( DestStride == Atlas.BytesPerPixel * Atlas.AtlasWidth ); // Temporarily disabling check
					FMemory::Memcpy( TempData, Atlas->AtlasData.GetData(), GetSlateFontAtlasContentBytesPerPixel(Atlas->GetContentType())*Atlas->AtlasWidth*Atlas->AtlasHeight );
					RHICmdList.UnlockTexture2D( Atlas->FontTexture->GetTypedResource(),0,false );
				});
		}

		bNeedsUpdate = false;
	}
}

FSlateFontTextureRHI::FSlateFontTextureRHI(const uint32 InWidth, const uint32 InHeight, ESlateFontAtlasContentType InContentType, const TArray<uint8>& InRawData)
	: FontTexture(new FSlateFontTextureRHIResource(InWidth, InHeight, InContentType))
{
	if (IsInRenderingThread())
	{
		FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

		FontTexture->InitResource(RHICmdList);
		UpdateTextureFromSource(RHICmdList, InWidth, InHeight, InRawData);
	}
	else
	{
		checkSlow(IsThreadSafeForSlateRendering());

		PendingSourceData.Reset(new FPendingSourceData(InWidth, InHeight, InRawData));

		BeginInitResource(FontTexture.Get());

		FSlateFontTextureRHI* This = this;
		ENQUEUE_RENDER_COMMAND(SlateUpdateFontTextureCommand)(
			[This](FRHICommandListImmediate& RHICmdList)
			{
				This->UpdateTextureFromSource(RHICmdList, This->PendingSourceData->SourceWidth, This->PendingSourceData->SourceHeight, This->PendingSourceData->SourceData);
				This->PendingSourceData.Reset();
			});
	}
}

FSlateFontTextureRHI::~FSlateFontTextureRHI()
{
}

void FSlateFontTextureRHI::GetAtlasDataCopy(TArray<uint8>& OutData) const
{
	OutData.Empty();
	// @TODOFonts: Implement 
}

void FSlateFontTextureRHI::ReleaseResources()
{
	checkSlow(IsThreadSafeForSlateRendering());

	BeginReleaseResource(FontTexture.Get());
}

void FSlateFontTextureRHI::UpdateTextureFromSource(FRHICommandListImmediate& RHICmdList, const uint32 SourceWidth, const uint32 SourceHeight, const TArray<uint8>& SourceData)
{
	const uint32 BytesPerPixel = GetSlateFontAtlasContentBytesPerPixel(GetContentType());

	uint32 DestStride;
	uint8* LockedTextureData = static_cast<uint8*>(RHICmdList.LockTexture2D(FontTexture->GetTypedResource(), 0, RLM_WriteOnly, /*out*/ DestStride, false));

	// If our stride matches our width, we can copy in a single call, rather than copy each line
	if (BytesPerPixel * SourceWidth == DestStride)
	{
		FMemory::Memcpy(LockedTextureData, SourceData.GetData(), BytesPerPixel * SourceWidth * SourceHeight);
	}
	else
	{
		for (uint32 RowIndex = 0; RowIndex < SourceHeight; ++RowIndex)
		{
			FMemory::Memcpy(LockedTextureData + (RowIndex * DestStride), SourceData.GetData() + (RowIndex * SourceWidth), BytesPerPixel * SourceWidth);
		}
	}
	
	RHICmdList.UnlockTexture2D(FontTexture->GetTypedResource(), 0, false);
}
