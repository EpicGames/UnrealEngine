// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Fonts/FontTypes.h"
#include "Textures/SlateShaderResource.h"
#include "TextureResource.h"

/**
 * Override for font textures that saves data between Init and ReleaseRHI to 
 * ensure all characters in the font texture exist if the rendering resource has to be recreated 
 * between caching new characters
 */
class FSlateFontTextureRHIResource : public TSlateTexture<FTextureRHIRef>, public FTextureResource
{
public:
	/** Constructor.  Initializes the texture
	 *
	 * @param InWidth The width of the texture
	 * @param InHeight The height of the texture
	 */
	FSlateFontTextureRHIResource(uint32 InWidth, uint32 InHeight, ESlateFontAtlasContentType InContentType);

	/** FSlateShaderResource interface */
	virtual uint32 GetWidth() const override { return Width; }
	virtual uint32 GetHeight() const override { return Height; }

	/** FTextureResource interface */
	virtual uint32 GetSizeX() const override { return Width; }
	virtual uint32 GetSizeY() const override { return Height; }
	virtual FString GetFriendlyName() const override { return TEXT("FSlateFontTextureRHIResource"); }

	/** FRenderResource interface */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;
	
	/** Returns texture content type */
	ESlateFontAtlasContentType GetContentType() const { return ContentType; }

private:
	EPixelFormat GetRHIPixelFormat() const;

	/** Width of this texture */
	uint32 Width;
	/** Height of this texture */
	uint32 Height;
	/** Type of content */
	ESlateFontAtlasContentType ContentType;
	/** Temporary data stored between Release and InitRHI */
	TArray<uint8> TempData;
};

/** 
 * Representation of a texture for fonts in which characters are packed tightly based on their bounding rectangle 
 */
class FSlateFontAtlasRHI : public FSlateFontAtlas
{
public:
	FSlateFontAtlasRHI(uint32 Width, uint32 Height, ESlateFontAtlasContentType InContentType, ESlateTextureAtlasPaddingStyle InPaddingStyle);
	~FSlateFontAtlasRHI();

	/**
	 * FSlateFontAtlas interface 
	 */
	virtual class FSlateShaderResource* GetSlateTexture() const override { return FontTexture.Get(); }
	virtual class FTextureResource* GetEngineTexture() override { return FontTexture.Get(); }
	virtual void ConditionalUpdateTexture()  override;
	virtual void ReleaseResources() override;
private:
	TUniquePtr<FSlateFontTextureRHIResource> FontTexture;
};

/**
 * A RHI non-atlased font texture resource
 */
class FSlateFontTextureRHI : public ISlateFontTexture
{
public:
	FSlateFontTextureRHI(const uint32 InWidth, const uint32 InHeight, ESlateFontAtlasContentType InContentType, const TArray<uint8>& InRawData);
	~FSlateFontTextureRHI();

	/**
	 * ISlateFontTexture interface 
	 */
	virtual class FSlateShaderResource* GetSlateTexture() const override { return FontTexture.Get(); }
	virtual class FTextureResource* GetEngineTexture() override { return FontTexture.Get(); }
	virtual ESlateFontAtlasContentType GetContentType() const override { return FontTexture->GetContentType(); }
	virtual void ReleaseRenderingResources() override
	{
		ReleaseResources();
	}
	virtual void GetAtlasDataCopy(TArray<uint8>& OutData) const override;

	void ReleaseResources();
private:
	void UpdateTextureFromSource(FRHICommandListImmediate& RHICmdList, const uint32 SourceWidth, const uint32 SourceHeight, const TArray<uint8>& SourceData);
private:
	struct FPendingSourceData
	{
		FPendingSourceData(uint32 InSourceWidth, uint32 InSourceHeight, TArray<uint8> InSourceData)
			: SourceWidth(InSourceWidth)
			, SourceHeight(InSourceHeight)
			, SourceData(MoveTemp(InSourceData))
		{
		}

		uint32 SourceWidth;
		uint32 SourceHeight;
		TArray<uint8> SourceData;
	};

	TUniquePtr<FPendingSourceData> PendingSourceData;

	TUniquePtr<FSlateFontTextureRHIResource> FontTexture;
};
