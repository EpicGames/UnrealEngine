// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateRHIRenderer.h"
#include "Fonts/FontCache.h"
#include "SlateRHIRenderingPolicy.h"
#include "SlateRHIRendererSettings.h"
#include "Misc/ScopeLock.h"
#include "Modules/ModuleManager.h"
#include "Styling/CoreStyle.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "EngineGlobals.h"
#include "Engine/AssetManager.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/UserInterfaceSettings.h"
#include "FX/SlateFXSubsystem.h"
#include "FX/SlateRHIPostBufferProcessor.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialShared.h"
#include "RendererInterface.h"
#include "StaticBoundShaderState.h"
#include "SceneInterface.h"
#include "SceneUtils.h"
#include "RHIStaticStates.h"
#include "UnrealEngine.h"
#include "GlobalShader.h"
#include "ScreenRendering.h"
#include "SlateShaders.h"
#include "Rendering/ElementBatcher.h"
#include "Rendering/SlateRenderer.h"
#include "RenderResource.h"
#include "RenderTimer.h"
#include "RenderingThread.h"
#include "RHIResources.h"
#include "RHIUtilities.h"
#include "StereoRendering.h"
#include "SlateNativeTextureResource.h"
#include "TextureResource.h"
#include "VolumeRendering.h"
#include "PipelineStateCache.h"
#include "EngineModule.h"
#include "Interfaces/ISlate3DRenderer.h"
#include "Interfaces/SlateRHIRenderingPolicyInterface.h"
#include "Slate/SlateTextureAtlasInterface.h"
#include "Types/ReflectionMetadata.h"
#include "CommonRenderResources.h"
#include "RenderTargetPool.h"
#include "RendererUtils.h"
#include "HAL/LowLevelMemTracker.h"
#include "Rendering/RenderingCommon.h"
#include "IHeadMountedDisplayModule.h"
#include "HDRHelper.h"
#include "RenderCore.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "SlatePostProcessor.h"
#include "Stats/ThreadIdleStats.h"
#include "VT/VirtualTextureFeedbackResource.h"
#include "Engine/RendererSettings.h"
#include "Slate/SlateViewportProvider.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"

#if WITH_EDITORONLY_DATA
#include "ShaderCompiler.h"
#endif

#if PLATFORM_MAC
#include "Misc/ScopeRWLock.h"
#endif

DECLARE_CYCLE_STAT(TEXT("Total Render Thread time including dependent waits"), STAT_RenderThreadCriticalPath, STATGROUP_Threading);

CSV_DEFINE_CATEGORY(RenderThreadIdle, true);
CSV_DECLARE_CATEGORY_MODULE_EXTERN(SLATECORE_API, Slate);

DECLARE_GPU_DRAWCALL_STAT_NAMED(SlateUI, TEXT("Slate UI"));

// Defines the maximum size that a slate viewport will create
#define MIN_VIEWPORT_SIZE 8
#define MAX_VIEWPORT_SIZE 16384

static TAutoConsoleVariable<float> CVarUILevel(
	TEXT("r.HDR.UI.Level"),
	1.0f,
	TEXT("Luminance level for UI elements when compositing into HDR framebuffer (default: 1.0)."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarHDRUILuminance(
	TEXT("r.HDR.UI.Luminance"),
	203.f,
	TEXT("Base Luminance in nits for UI elements when compositing into HDR framebuffer. Gets multiplied by r.HDR.UI.Level"),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarHDRUILuminanceMode(
	TEXT("r.HDR.UI.Luminance.Mode"),
	0,
	TEXT("Specifies how UI content incorporates paper white values.\n")
	TEXT("0: Use the paper white value according to r.HDR.PaperWhite.Mode (default)\n")
	TEXT("1: Use r.HDR.UI.Luminance"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHDRUICompositeEOTF(
	TEXT("r.HDR.UI.CompositeEOTF"),
	0,
	TEXT("EOTF used when compositing the UI layer:\n")
	TEXT("0: Gamma (default), see r.HDR.UI.CompositeEOTF.Gamma\n")
	TEXT("1: sRGB"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarHDRUICompositeEOTFGamma(
	TEXT("r.HDR.UI.CompositeEOTF.Gamma"),
	2.4f,
	TEXT("The power used if using gamma to composite the UI layer:\n")
	TEXT("Defaults to 2.4 for BT.1886 specified by BT.2408."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarHDRUICompositeBlend(
	TEXT("r.HDR.UI.CompositeBlend"),
	0,
	TEXT("The blend space used when compositing the UI layer:\n")
	TEXT("0: Gamma (default), see r.HDR.UI.CompositeBlend.Gamma\n")
	TEXT("1: Linear"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarHDRUICompositeBlendGamma(
	TEXT("r.HDR.UI.CompositeBlend.Gamma"),
	2.4f,
	TEXT("The power used if using gamma to composite the UI layer:\n")
	TEXT("Defaults to 2.4 to match r.HDR.UI.CompositeEOTF.Gamma."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarUICompositeMode(
	TEXT("r.HDR.UI.CompositeMode"),
	1,
	TEXT("Mode used when compositing the UI layer:\n")
	TEXT("0: Standard compositing\n")
	TEXT("1: Shader pass to improve HDR blending\n"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarCopyBackbufferToSlatePostRenderTargets(
	TEXT("Slate.CopyBackbufferToSlatePostRenderTargets"),
	0,
	TEXT("Experimental. Set true to copy final backbuffer into slate RTs for slate post processing / material usage"),
	ECVF_RenderThreadSafe);

#if WITH_SLATE_VISUALIZERS

TAutoConsoleVariable<int32> CVarShowSlateOverdraw(
	TEXT("Slate.ShowOverdraw"),
	0,
	TEXT("0: Don't show overdraw, 1: Show Overdraw"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarShowSlateBatching(
	TEXT("Slate.ShowBatching"),
	0,
	TEXT("0: Don't show batching, 1: Show Batching"),
	ECVF_RenderThreadSafe
);
#endif

bool GSlateWireframe = false;
static FAutoConsoleVariableRef CVarSlateWireframe(TEXT("Slate.ShowWireFrame"), GSlateWireframe, TEXT(""), ECVF_Default);

// RT stat including waits toggle. Off by default for historical tracking reasons
static TAutoConsoleVariable<int32> CVarRenderThreadTimeIncludesDependentWaits(
	TEXT("r.RenderThreadTimeIncludesDependentWaits"),
	0,
	TEXT("0: RT stat only includes non-idle time, 1: RT stat includes dependent waits (matching RenderThreadTime_CriticalPath)"),
	ECVF_Default
);

#if WITH_SLATE_DEBUGGING
static TAutoConsoleVariable<bool> CVarSlateDumpNumDefaultPostBufferUpdates(
	TEXT("Slate.DumpNumDefaultPostBufferUpdates"),
	false,
	TEXT("Dump number of slate default post buffer updates in a frame. Updates every 60f. See also: Slate.DumpNumWidgetPostBufferUpdates.")
);
#endif // WITH_SLATE_DEBUGGING

static bool IsVSyncRequired(const FSlateElementBatcher& ElementBatcher)
{
	bool bLockToVsync = ElementBatcher.RequiresVsync();

	if (GIsEditor)
	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.VSyncEditor"));
		bLockToVsync |= (CVar->GetInt() != 0);
	}
	else
	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.VSync"));
		bLockToVsync |= (CVar->GetInt() != 0);
	}

	return bLockToVsync;
}

static bool ShouldPresent(const bool IsCameraCut)
{
	bool bPresent = true;

	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SkipPresentOnCameraCut"));
		if ((CVar->GetInt() != 0) && IsCameraCut)
		{
			bPresent = false;
		}
	}

	return bPresent;
}

FMatrix CreateSlateProjectionMatrix(uint32 Width, uint32 Height)
{
	// Create ortho projection matrix
	const float Left = 0;
	const float Right = Left + Width;
	const float Top = 0;
	const float Bottom = Top + Height;
	const float ZNear = -100.0f;
	const float ZFar = 100.0f;
	return AdjustProjectionMatrixForRHI(
		FMatrix(
			FPlane(2.0f / (Right - Left), 0, 0, 0),
			FPlane(0, 2.0f / (Top - Bottom), 0, 0),
			FPlane(0, 0, 1 / (ZNear - ZFar), 0),
			FPlane((Left + Right) / (Left - Right), (Top + Bottom) / (Bottom - Top), ZNear / (ZNear - ZFar), 1)
		)
	);
}

struct FSlateViewportInfo final : public FRenderResource, public ISlateViewportProvider
{
	struct FLayerInfo
	{
		TSet<int32> NativeLayers;
		TArray<TInterval<int32>> Ranges;
	} Layers;

	FSlateRHIRenderer& Renderer;
	TSharedRef<SWindow> Window;
	TVariant<FViewportRHIRef, FTextureRHIRef> ViewportOrTextureRHI;

	EPixelFormat PixelFormat = EPixelFormat::PF_Unknown;
	FIntPoint Extent = FIntPoint::ZeroValue;
	FMatrix ProjectionMatrix;

	bool bFullscreen = false;

	FHDRMetaData HDRMetaData;

	FSlateViewportInfo(FSlateRHIRenderer& InRenderer, const TSharedRef<SWindow> InWindow, FIntPoint InExtent);

	bool IsDisplayFormatHDR() const;
	void ResizeViewport(FIntPoint ExtentToResizeTo, bool bInFullscreen);
	void ReleaseRHI() override;

	bool IsViewportRHI() const
	{
		return ViewportOrTextureRHI.IsType<FViewportRHIRef>();
	}

	bool IsTextureRHI() const
	{
		return !IsViewportRHI();
	}

	FViewportRHIRef      & GetViewportRHI()       { return ViewportOrTextureRHI.Get<FViewportRHIRef>(); }
	FViewportRHIRef const& GetViewportRHI() const { return ViewportOrTextureRHI.Get<FViewportRHIRef>(); }
	FTextureRHIRef       & GetTextureRHI ()       { return ViewportOrTextureRHI.Get<FTextureRHIRef >(); }
	FTextureRHIRef const & GetTextureRHI () const { return ViewportOrTextureRHI.Get<FTextureRHIRef >(); }

	FRHITexture* GetBackBufferResource(TOptional<int32> LayerIndex) const;
	FRHITexture* GetBackBufferResource() const override;
	void SetCustomPresent(FRHICustomPresent* CustomPresent) override;
	void SetWindowCropSize(FIntRect& NewCropWindow) override;

	void CreateNativeLayer(int32 NativeLayer, void* NativeViewHandle);
	void DeleteNativeLayer(int32 NativeLayer);

private:
	static FIntPoint ClampExtent(FIntPoint Extent)
	{
		// Windows are allowed to be zero sized (sometimes they are animating to/from zero for example) but not viewports.
		Extent = Extent.ComponentMax(FIntPoint(MIN_VIEWPORT_SIZE, MIN_VIEWPORT_SIZE));

		if (Extent.X > MAX_VIEWPORT_SIZE)
		{
			UE_LOGF(LogSlate, Warning, "Tried to set viewport width size to %d.  Clamping size to max allowed size of %d instead.", Extent.X, MAX_VIEWPORT_SIZE);
			Extent.X = MAX_VIEWPORT_SIZE;
		}

		if (Extent.Y > MAX_VIEWPORT_SIZE)
		{
			UE_LOGF(LogSlate, Warning, "Tried to set viewport height size to %d.  Clamping size to max allowed size of %d instead.", Extent.Y, MAX_VIEWPORT_SIZE);
			Extent.Y = MAX_VIEWPORT_SIZE;
		}

		return Extent;
	}

	void CreateTexture2D();
};

FSlateViewportInfo::FSlateViewportInfo(FSlateRHIRenderer& InRenderer, const TSharedRef<SWindow> InWindow, FIntPoint InExtent)
	: Renderer(InRenderer)
	, Window(InWindow)
	, ViewportOrTextureRHI(TInPlaceType<FViewportRHIRef>(), nullptr)
	, Extent(ClampExtent(InExtent))
	, ProjectionMatrix(CreateSlateProjectionMatrix(Extent.X, Extent.Y))
	, bFullscreen(Renderer.IsViewportFullscreen(*Window))
{
	void* OSWindow = Window->GetNativeWindow()->GetOSWindowHandle();

	HDRGetMetaDataForWindow(HDRMetaData, Window->GetPositionInScreen(), Window->GetPositionInScreen() + Window->GetSizeInScreen(), OSWindow);

	PixelFormat = Renderer.GetViewportPixelFormat(*Window, HDRMetaData.bHDRSupported);

	if (Window->GetNativeWindow()->IsRenderingOffScreen())
	{
		ViewportOrTextureRHI.Emplace<FTextureRHIRef>();
		CreateTexture2D();
	}
	else
	{
		GetViewportRHI() = RHICreateViewport(OSWindow, Extent.X, Extent.Y, bFullscreen, PixelFormat);
	}

	BeginInitResource(this);

	Window->SetIsHDR(IsDisplayFormatHDR());

	// Create the very base "native layer" used exclusively if we need no interoperability with native UI elements.
	// (i.e. only rendering to one native view for Slate)
	CreateNativeLayer(std::numeric_limits<int32>::min(), nullptr);
}

void FSlateViewportInfo::CreateTexture2D()
{
	check(IsTextureRHI());

	ENQUEUE_RENDER_COMMAND(CreateOffScreenTexture)([this](FRHICommandListImmediate& RHICmdList)
	{
		FRHITextureCreateDesc CreateDesc = FRHITextureCreateDesc::Create2D(TEXT("SlateOffScreenViewportTexture"), Extent, PixelFormat)
			.SetFlags(ETextureCreateFlags::ShaderResource | ETextureCreateFlags::RenderTargetable)
			.SetClearValue(FClearValueBinding::Transparent)
			.SetInitialState(ERHIAccess::SRVMask);

		GetTextureRHI() = RHICmdList.CreateTexture(CreateDesc);
	});

	FlushRenderingCommands();
}

void FSlateViewportInfo::CreateNativeLayer(int32 NativeLayer, void* NativeViewHandle)
{
	// this flush is needed to make sure that the rendering thread is finished and not
	// using any of the NativeLayer resources to prevent race conditions.
	FlushRenderingCommands();
	
	bool bAlreadyCreated = false;
	Layers.NativeLayers.Add(NativeLayer, &bAlreadyCreated);

	// Rebuild the layer ranges
	TArray<int32> NativeLayersSorted = Layers.NativeLayers.Array();
	NativeLayersSorted.Sort([](const int32& A, const int32& B) 
	{
		return A > B;
	});
	
	Layers.Ranges.Empty();
	for (int32 Index = 0; NativeLayersSorted.Num() > Index; ++Index)
	{
		TInterval<int32>& Range = Layers.Ranges.Emplace_GetRef();

		if (Index == 0)
		{
			Range.Min = NativeLayersSorted[Index];
			Range.Max = std::numeric_limits<int32>::max();
		}
		else if (Index == Layers.NativeLayers.Num() - 1)
		{
			check(NativeLayersSorted[Index] == std::numeric_limits<int32>::min());
				
			Range.Min = std::numeric_limits<int32>::min();
			Range.Max = NativeLayersSorted[Index - 1];
		}
		else
		{
			Range.Min = NativeLayersSorted[Index];
			Range.Max = NativeLayersSorted[Index - 1];
		}
	}

	if (!bAlreadyCreated)
	{
		if (IsViewportRHI())
		{
			GetViewportRHI()->CreateNativeLayer(NativeLayer, NativeViewHandle, Layers.Ranges);
		}
		else
		{
			// @todo - Not currently implemented for off-screen rendering
			checkf(NativeLayer == std::numeric_limits<int32>::min(), TEXT("Layered viewports are not currently implemented for off-screen rendering."));
		}
	}
}

void FSlateViewportInfo::DeleteNativeLayer(int32 NativeLayer)
{
	// this flush is needed to make sure that the rendering thread is finished and not
	// using any of the NativeLayer resources to prevent race conditions.
	FlushRenderingCommands();
	
	// Make sure we aren't deleting the bottom most layer
	check(NativeLayer != std::numeric_limits<int32>::min());

	check(Layers.NativeLayers.Contains(NativeLayer));
	Layers.NativeLayers.Remove(NativeLayer);

	if (IsViewportRHI())
	{
		GetViewportRHI()->DeleteNativeLayer(NativeLayer);
	}
	else
	{
		// @todo - Not currently implemented for off-screen rendering
		checkf(false, TEXT("Layered viewports are not currently implemented for off-screen rendering."));
	}

	bool FoundLayerRangeToRemove = false;

	// Skip the lowest layer, we can't delete it
	for (int32 Index = Layers.Ranges.Num() - 2; 0 <= Index; --Index)
	{
		if (NativeLayer == Layers.Ranges[Index].Min)
		{
			// these are reverse sorted (lowest layer, highest layer)
			check(Layers.Ranges[Index + 1].Max == NativeLayer);

			Layers.Ranges[Index + 1].Max = Layers.Ranges[Index].Max;
			Layers.Ranges.RemoveAt(Index);
			FoundLayerRangeToRemove = true;

			break;
		}
	}

	check(FoundLayerRangeToRemove);
}

FRHITexture* FSlateViewportInfo::GetBackBufferResource(TOptional<int32> LayerIndex) const
{
	check(IsInRenderingThread());
	if (IsViewportRHI())
	{
		if (Layers.NativeLayers.Num() > 1)
		{
			return LayerIndex.IsSet()
				? RHIGetViewportBackBufferAtLayerIndex(GetViewportRHI(), LayerIndex.GetValue())
				: RHIGetViewportBackBuffer(GetViewportRHI());
		}
		else
		{
			check(!LayerIndex.IsSet() || LayerIndex.GetValue() == std::numeric_limits<int32>::min());
			return RHIGetViewportBackBuffer(GetViewportRHI());
		}
	}
	else
	{
		checkf(!LayerIndex.IsSet() || LayerIndex.GetValue() == std::numeric_limits<int32>::min(), TEXT("Layers not currently implemented for off-screen rendering"));
		return GetTextureRHI();
	}
}

FRHITexture* FSlateViewportInfo::GetBackBufferResource() const
{
	return GetBackBufferResource({});
}

void FSlateViewportInfo::SetCustomPresent(FRHICustomPresent* CustomPresent)
{
	check(IsInRenderingThread());
	if (IsViewportRHI())
	{
		FRHICommandListImmediate::Get().EnqueueLambda(TEXT("SetCustomPresent"),
			[ViewportRHI = GetViewportRHI(), CustomPresent](FRHICommandListImmediate&)
			{
				ViewportRHI->SetCustomPresent(CustomPresent);
			});
	}
}

void FSlateViewportInfo::SetWindowCropSize(FIntRect& NewCropWindow)
{
	check(IsInRenderingThread());
	if (IsViewportRHI())
	{
		FRHICommandListImmediate::Get().EnqueueLambda(TEXT("SetWindowCropSize"),
			[ViewportRHI = GetViewportRHI(), RHINewCropWindow = NewCropWindow](FRHICommandListImmediate&)
			{
				ViewportRHI->SetWindowCropSize(RHINewCropWindow);
			});
	}
}

void FSlateViewportInfo::ReleaseRHI()
{
	// Full GPU sync here to simplify memory lifetime of the underlying resource.
	FRHICommandListImmediate::Get().SubmitAndBlockUntilGPUIdle();

	if (IsViewportRHI())
	{
		GetViewportRHI() = nullptr;
	}
	else
	{
		GetTextureRHI() = nullptr;
	}
}

bool FSlateViewportInfo::IsDisplayFormatHDR() const
{
	return HDRMetaData.bHDRSupported;
}

void FSlateViewportInfo::ResizeViewport(FIntPoint ExtentToResizeTo, bool bInFullscreen)
{
	check(IsThreadSafeForSlateRendering());
	if (!IsInGameThread() || IsInSlateThread())
	{
		return;
	}

	ExtentToResizeTo = ClampExtent(ExtentToResizeTo);

	FHDRMetaData NewHDRMetaData;
	HDRGetMetaDataForWindow(NewHDRMetaData, Window->GetPositionInScreen(), Window->GetPositionInScreen() + Window->GetSizeInScreen(), Window->GetNativeWindow()->GetOSWindowHandle());

	bool bHDRStale = false;
	bHDRStale |= NewHDRMetaData.DisplayOutputFormat != HDRMetaData.DisplayOutputFormat;
	bHDRStale |= NewHDRMetaData.DisplayColorGamut   != HDRMetaData.DisplayColorGamut;
	bHDRStale |= NewHDRMetaData.HDRPaperWhiteInNits != HDRMetaData.HDRPaperWhiteInNits;
	bHDRStale |= NewHDRMetaData.bHDRSupported       != HDRMetaData.bHDRSupported;

	if (bHDRStale || Extent != ExtentToResizeTo || bFullscreen != bInFullscreen)
	{
		// Prevent the texture update logic to use the RHI while the viewport is resized. 
		// This could happen if a streaming IO request completes and throws a callback.
		// @todo : this does not in fact stop texture tasks from using the RHI while the viewport is resized
		//		because they can be running in other threads, or even in retraction on this thread inside the D3D Wait
		//		this should be removed and whatever streaming thread safety is needed during a viewport resize should be done correctly
		SuspendTextureStreamingRenderTasks();

		// Wait for any pending async cleanup
		ENQUEUE_RENDER_COMMAND(FAsyncCleanup)([](FRHICommandListImmediate&)
		{
			FRDGBuilder::WaitForAsyncDeleteTask();
		});

		// cannot resize the viewport while potentially using it.
		FlushRenderingCommands();

		Extent = ExtentToResizeTo;
		ProjectionMatrix = CreateSlateProjectionMatrix(Extent.X, Extent.Y);
		bFullscreen = bInFullscreen;
		HDRMetaData = NewHDRMetaData;

		PixelFormat = Renderer.GetViewportPixelFormat(*Window, HDRMetaData.bHDRSupported);

		if (IsViewportRHI())
		{
			ensureMsgf(GetViewportRHI()->GetRefCount() == 1, TEXT("Viewport backbuffer was not properly released"));
			RHIResizeViewport(GetViewportRHI(), Extent.X, Extent.Y, bFullscreen, PixelFormat);
		}
		else
		{
			CreateTexture2D();
		}

		// Reset texture streaming texture updates.
		ResumeTextureStreamingRenderTasks();

		// when the window's state for HDR changed, we need to invalidate the window to make sure the viewport will end up in the appropriate FSlateBatchData, see FSlateElementBatcher::AddViewportElement
		if (bHDRStale)
		{
			Window->Invalidate(EInvalidateWidgetReason::Paint);
		}
	}
}

struct FSlateDrawWindowPassInputs
{
	FSlateRHIRenderer* Renderer = nullptr;
	FSlateWindowElementList* WindowElementList = nullptr;
	SWindow* Window = nullptr;
	FSlateViewportInfo* ViewportInfo = nullptr;
	FIntPoint CursorPosition = FIntPoint::ZeroValue;
	FIntRect SceneViewRect;
	float ViewportScaleUI = 0.0f;
	ESlatePostRT UsedSlatePostBuffers = ESlatePostRT::None;
#if WANTS_DRAW_MESH_EVENTS
	FString WindowTitle;
#endif
	FGameTime Time;
	bool bLockToVsync = false;
	bool bClear = false;
	bool bPresent = true;
};

struct FSlateDrawWindowPassOutputs
{
	FSlateViewportInfo& ViewportInfo;
	FRHITexture* ViewportTextureRHI = nullptr;
	FRHITexture* OutputTextureRHI = nullptr;
};

FSlateRHIRenderer::FSlateRHIRenderer(TSharedRef<FSlateFontServices> InSlateFontServices, TSharedRef<FSlateRHIResourceManager> InResourceManager)
	: FSlateRenderer(InSlateFontServices)
	, ResourceManager(InResourceManager)
	, bIsStandaloneStereoOnlyDevice(IHeadMountedDisplayModule::IsAvailable() && IHeadMountedDisplayModule::Get().IsStandaloneStereoOnlyDevice())
{
	for (uint64& LastFramePostBufferUsed : LastFramesPostBufferUsed)
	{
		LastFramePostBufferUsed = 0;
	}
}

bool FSlateRHIRenderer::Initialize()
{
	LoadUsedTextures();

	RenderingPolicy = MakeShareable(new FSlateRHIRenderingPolicy(SlateFontServices.ToSharedRef(), ResourceManager.ToSharedRef()));

	ElementBatcher = MakeUnique<FSlateElementBatcher>(RenderingPolicy.ToSharedRef());

	CurrentSceneIndex = -1;
	ActiveScenes.Empty();

	if (!SceneRemovedHandle.IsValid())
	{
		SceneRemovedHandle = GetRendererModule().GetSceneRemovedEvent().AddRaw(this, &FSlateRHIRenderer::OnSceneRemoved);
	}
	return true;
}

void FSlateRHIRenderer::Destroy()
{
	if (SceneRemovedHandle.IsValid())
	{
		GetRendererModule().GetSceneRemovedEvent().Remove(SceneRemovedHandle);
		SceneRemovedHandle.Reset();
	}

	ResourceManager->ReleaseResources();
	SlateFontServices->ReleaseResources();

	for (auto& Entry : WindowToViewportInfo)
	{
		BeginReleaseResource(Entry.Value);
	}

	FlushPendingDeletes();
	FlushRenderingCommands();

	ElementBatcher.Reset();
	RenderingPolicy.Reset();
	ResourceManager.Reset();
	SlateFontServices.Reset();

	DeferredUpdateContexts.Empty();

	for (auto& Entry : WindowToViewportInfo)
	{
		delete Entry.Value;
	}
	WindowToViewportInfo.Empty();

	CurrentSceneIndex = -1;
	ActiveScenes.Empty();
}

/** Returns a draw buffer that can be used by Slate windows to draw window elements */
FSlateDrawBuffer& FSlateRHIRenderer::AcquireDrawBuffer()
{
	FreeBufferIndex = (FreeBufferIndex + 1) % NumDrawBuffers;

	FSlateDrawBuffer* Buffer = &DrawBuffers[FreeBufferIndex];

	while (!Buffer->Lock())
	{
		// If the buffer cannot be locked then the buffer is still in use.  If we are here all buffers are in use
		// so wait until one is free.
		if (IsInSlateThread())
		{
			// We can't flush commands on the slate thread, so simply spinlock until we're done
			// this happens if the render thread becomes completely blocked by expensive tasks when the Slate thread is running
			// in this case we cannot tick Slate.
			FPlatformProcess::Sleep(0.001f);
		}
		else
		{
			FlushCommands();
			FreeBufferIndex = (FreeBufferIndex + 1) % NumDrawBuffers;
		}


		Buffer = &DrawBuffers[FreeBufferIndex];
	}

	// Safely remove brushes by emptying the array and releasing references
	DynamicBrushesToRemove[FreeBufferIndex].Empty();

	Buffer->ClearBuffer();
	Buffer->UpdateResourceVersion(ResourceVersion);
	return *Buffer;
}

void FSlateRHIRenderer::ReleaseDrawBuffer(FSlateDrawBuffer& WindowDrawBuffer)
{
#if DO_CHECK
	bool bFound = false;
	for (int32 Index = 0; Index < NumDrawBuffers; ++Index)
	{
		if (&DrawBuffers[Index] == &WindowDrawBuffer)
		{
			bFound = true;
			break;
		}
	}
	ensureMsgf(bFound, TEXT("It release a DrawBuffer that is not a member of the SlateRHIRenderer"));
#endif

	ENQUEUE_RENDER_COMMAND(SlateReleaseDrawBufferCommand)([&WindowDrawBuffer](FRHICommandList& RHICmdList)
	{
		WindowDrawBuffer.Unlock(FRDGBuilder::GetAsyncExecuteTask());
	});
}

void FSlateRHIRenderer::CreateViewport(const TSharedRef<SWindow> Window)
{
	if (WindowToViewportInfo.Contains(&Window.Get()))
	{
		return;
	}

	FlushRenderingCommands();

	const FVector2f ViewportSize = Window->GetViewportSize();

	FIntPoint Extent;
	Extent.X = FMath::CeilToInt(ViewportSize.X);
	Extent.Y = FMath::CeilToInt(ViewportSize.Y);

	WindowToViewportInfo.Add(&Window.Get(), new FSlateViewportInfo(*this, Window, Extent));
}

EPixelFormat FSlateRHIRenderer::GetViewportPixelFormat(const SWindow& Window, bool bDisplayFormatIsHDR)
{
	// Use the configured HDR format if enabled.
	if (bDisplayFormatIsHDR)
	{
		return GRHIHDRDisplayOutputFormat;
	}

	// Use a known default format in VR / Mobile / Transparent Window SDR configurations.
	if (bIsStandaloneStereoOnlyDevice
#if ALPHA_BLENDED_WINDOWS
		|| ((Window.GetTransparencySupport() == EWindowTransparency::PerPixel) && !IsDefaultBackBufferLinearSDR())
#endif
		)
	{
		return GetSlateRecommendedColorFormat();
	}

	// Use the renderer default.
	static const auto CVarDefaultBackBufferPixelFormat = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DefaultBackBufferPixelFormat"));
	return EDefaultBackBufferPixelFormat::Convert2PixelFormat(EDefaultBackBufferPixelFormat::FromInt(CVarDefaultBackBufferPixelFormat->GetValueOnGameThread()));
}

void FSlateRHIRenderer::OnVirtualDesktopSizeChanged(const FDisplayMetrics& NewDisplayMetric)
{
	// Defer the update to as we need to call FlushRenderingCommands() before sending the event to the RHI. 
	// FlushRenderingCommands -> FRenderCommandFence::IsFenceComplete -> CheckRenderingThreadHealth -> FPlatformApplicationMisc::PumpMessages
	// The Display change event is not been consumed yet, and we do BroadcastDisplayMetricsChanged -> OnVirtualDesktopSizeChanged again
	bUpdateHDRDisplayInformation = true;
}

void FSlateRHIRenderer::UpdateFullscreenState(const TSharedRef<SWindow> Window, uint32 OverrideResX, uint32 OverrideResY)
{
	FSlateViewportInfo* ViewInfo = WindowToViewportInfo.FindRef(&Window.Get());

	if (!ViewInfo)
	{
		CreateViewport(Window);
	}

	ViewInfo = WindowToViewportInfo.FindRef(&Window.Get());

	if (ViewInfo)
	{
		const bool bFullscreen = IsViewportFullscreen(*Window);
		const bool bIsRenderingStereo = GEngine && GEngine->XRSystem.IsValid() && GEngine->StereoRenderingDevice.IsValid() && GEngine->StereoRenderingDevice->IsStereoEnabled();

		FIntPoint ExtentToResizeTo(OverrideResX ? OverrideResX : GSystemResolution.ResX, OverrideResY ? OverrideResY : GSystemResolution.ResY);

		if ((GIsEditor && Window->IsViewportSizeDrivenByWindow()) || (Window->GetWindowMode() == EWindowMode::WindowedFullscreen) || bIsRenderingStereo)
		{
			ExtentToResizeTo = ViewInfo->Extent;
		}

		ViewInfo->ResizeViewport(ExtentToResizeTo, bFullscreen);
	}
}

void FSlateRHIRenderer::SetSystemResolution(uint32 Width, uint32 Height)
{
	FSystemResolution::RequestResolutionChange(Width, Height, FPlatformProperties::HasFixedResolution() ? EWindowMode::Fullscreen : GSystemResolution.WindowMode);
	IConsoleManager::Get().CallAllConsoleVariableSinks();
}

void FSlateRHIRenderer::RestoreSystemResolution(const TSharedRef<SWindow> InWindow)
{
	if (!GIsEditor && InWindow->GetWindowMode() == EWindowMode::Fullscreen)
	{
		// Force the window system to resize the active viewport, even though nothing might have appeared to change.
		// On windows, DXGI might change the window resolution behind our backs when we alt-tab out. This will make
		// sure that we are actually in the resolution we think we are.
		GSystemResolution.ForceRefresh();
	}
}

void FSlateRHIRenderer::OnWindowDestroyed(const TSharedRef<SWindow>& InWindow)
{
	if (FSlateViewportInfo** ViewportInfoPtr = WindowToViewportInfo.Find(&InWindow.Get()))
	{
		FSlateViewportInfo* ViewportInfo = *ViewportInfoPtr;

		// Perform the release in lock-step with the render thread to simplify resource lifetimes.
		FlushRenderingCommands();
		BeginReleaseResource(ViewportInfo);
		FlushRenderingCommands();
		delete ViewportInfo;

		WindowToViewportInfo.Remove(&InWindow.Get());

		// Final flush to ensure any in-flight RHI present commands for this viewport have completed
		// before the caller destroys the native window. On Wayland, SDL_DestroyWindow immediately
		// invalidates the wl_surface, and any subsequent Vulkan present to that surface will crash.
		FlushRenderingCommands();
	}
}

// Limited platform support for HDR UI composition
bool SupportsCompositeUIWithSceneHDR(const EShaderPlatform Platform)
{
	return IsFeatureLevelSupported(Platform, ERHIFeatureLevel::SM5) && (RHISupportsGeometryShaders(Platform) || RHISupportsVertexShaderLayer(Platform));
}

bool CompositeUIWithSceneHDR()
{
	return GRHISupportsHDROutput
		&& RHISupportsVolumeTextureRendering(GetFeatureLevelShaderPlatform(GMaxRHIFeatureLevel))
		&& SupportsCompositeUIWithSceneHDR(GetFeatureLevelShaderPlatform(GMaxRHIFeatureLevel))
		&& CVarUICompositeMode.GetValueOnAnyThread() != 0;
}

BEGIN_SHADER_PARAMETER_STRUCT(FCompositeShaderCommonParameters, )
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, UITexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, UISampler)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, UIWriteMaskTexture)
	SHADER_PARAMETER(float, UILevel)
	SHADER_PARAMETER(float, UILuminance)
	SHADER_PARAMETER(int32, UICompositeEOTF)
	SHADER_PARAMETER(float, UICompositeEOTFGamma)
	SHADER_PARAMETER(int32, UICompositeBlend)
	SHADER_PARAMETER(float, UICompositeBlendGamma)
	SHADER_PARAMETER(float, UICompositeBlendInvGamma)
	SHADER_PARAMETER(float, ColorVisionDeficiencyType)
	SHADER_PARAMETER(float, ColorVisionDeficiencySeverity)
	SHADER_PARAMETER(float, bCorrectDeficiency)
	SHADER_PARAMETER(float, bSimulateCorrectionWithDeficiency)
END_SHADER_PARAMETER_STRUCT()

class FCompositeShader : public FGlobalShader
{
public:
	class FSCRGBEncoding : SHADER_PERMUTATION_BOOL("SCRGB_ENCODING");
	class FApplyColorDeficiency : SHADER_PERMUTATION_BOOL("APPLY_COLOR_DEFICIENCY");
	using FPermutationDomain = TShaderPermutationDomain<FSCRGBEncoding, FApplyColorDeficiency>;

	FCompositeShader() {}

	FCompositeShader(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return SupportsCompositeUIWithSceneHDR(Parameters.Platform);
	}
};

class FCompositePS : public FCompositeShader
{
public:
	DECLARE_GLOBAL_SHADER(FCompositePS);
	SHADER_USE_PARAMETER_STRUCT(FCompositePS, FCompositeShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FCompositeShaderCommonParameters, Common)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SceneSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FCompositePS, "/Engine/Private/CompositeUIPixelShader.usf", "Main", SF_Pixel);

class FCompositeCS : public FCompositeShader
{
public:
	static const uint32 NUM_THREADS_PER_GROUP = 16;

	DECLARE_GLOBAL_SHADER(FCompositeCS);
	SHADER_USE_PARAMETER_STRUCT(FCompositeCS, FCompositeShader);
	
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FCompositeShaderCommonParameters, Common)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWSceneTexture)
		SHADER_PARAMETER(FVector4f, SceneTextureDimensions)
	END_SHADER_PARAMETER_STRUCT()

	static bool IsShaderSupported(const EShaderPlatform ShaderPlatform)
	{
		return RHISupports4ComponentUAVReadWrite(ShaderPlatform) && RHISupportsSwapchainUAVs(ShaderPlatform);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return FCompositeShader::ShouldCompilePermutation(Parameters) && IsShaderSupported(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("USE_COMPUTE_FOR_COMPOSITION"), 1);
		OutEnvironment.SetDefine(TEXT("NUM_THREADS_PER_GROUP"), NUM_THREADS_PER_GROUP);
	}
};

IMPLEMENT_GLOBAL_SHADER(FCompositeCS, "/Engine/Private/CompositeUIPixelShader.usf", "CompositeUICS", SF_Compute);

void FSlateRHIRenderer::DrawWindowViewport_RenderThread(
	ISlateViewport* SlateViewport, FRDGBuilder& GraphBuilder, const FSlateDrawWindowPassInputs& Inputs,
	FRHITexture** ViewportTextureRHI, FRHITexture** OutputTextureRHI, FSlateViewportInfo& ViewportInfo,
	FSlateBatchData& BatchData, FSlateBatchData& BatchDataHDR,
	TOptional<int32> NativeLayer, TInterval<int32> LayerRange)
{
	// The viewport texture is an optional user-allocated render target. This is rendered to if valid.
	*ViewportTextureRHI = SlateViewport && SlateViewport->UseSeparateRenderTarget() ? static_cast<FSlateRenderTargetRHI*>(SlateViewport->GetViewportRenderTargetTexture())->GetTypedResource() : nullptr;

	// The swap chain is the final output. This is rendered to if no viewport render target is provided.
	FRHITexture* SwapChainTextureRHI = ViewportInfo.GetBackBufferResource(NativeLayer);

	// Only render to the intermediate viewport render target if stereo rendering is enabled, which we'll then composite later.
	const bool bCompositeStereoToSwapChain = *ViewportTextureRHI && GEngine && GEngine->StereoRenderingDevice.IsValid() && SlateViewport && SlateViewport->IsStereoscopic3D();
	
	// The output texture is what we ultimately render or composite slate elements into.
	*OutputTextureRHI = bCompositeStereoToSwapChain ? *ViewportTextureRHI : SwapChainTextureRHI;
	
	const TCHAR* OutputTextureName;
	if (NativeLayer.IsSet())
	{
		OutputTextureName = GraphBuilder.AllocObject<FString>(FString::Printf(TEXT("SlateOutputTexture-%d"), NativeLayer.GetValue()))->GetCharArray().GetData();
	}
	else
	{
		OutputTextureName = TEXT("SlateOutputTexture");
	}

	FRDGTexture* OutputTexture = RegisterExternalTexture(GraphBuilder, *OutputTextureRHI, OutputTextureName);

	// The elements texture contains UI elements. It can be the same as the output or allocated separately and composited.
	FRDGTexture* ElementsTexture = OutputTexture;
	const FIntPoint OutputExtent = OutputTexture->Desc.Extent;

	const bool bCompositeUIWithSceneHDR = ViewportInfo.IsDisplayFormatHDR() && CompositeUIWithSceneHDR();

	bool bClearElementsTexture = Inputs.bClear || GSlateWireframe;

#if WITH_SLATE_VISUALIZERS
	bClearElementsTexture |= CVarShowSlateBatching.GetValueOnRenderThread() != 0 || CVarShowSlateOverdraw.GetValueOnRenderThread() != 0;
#endif

	if (bCompositeUIWithSceneHDR)
	{
		const ETextureCreateFlags WriteMaskFlags = RHISupportsRenderTargetWriteMask(GMaxRHIShaderPlatform) ? ETextureCreateFlags::NoFastClearFinalize | ETextureCreateFlags::DisableDCC : ETextureCreateFlags::None;

		ElementsTexture = GraphBuilder.CreateTexture(
			FRDGTextureDesc::Create2D(
				OutputExtent,
				GetSlateRecommendedColorFormat(),
				FClearValueBinding::Transparent,
				ETextureCreateFlags::ShaderResource | ETextureCreateFlags::RenderTargetable | WriteMaskFlags),
			TEXT("CompositeUIWithSceneHDRTexture"));

		// Force a clear of the UI elements texture to black
		bClearElementsTexture = true;
	}

	const bool bRequiresVirtualTextureFeedback = BatchData.IsVirtualTextureFeedbackRequired() || BatchDataHDR.IsVirtualTextureFeedbackRequired();
	if (bRequiresVirtualTextureFeedback)
	{
		VirtualTexture::BeginFeedback(GraphBuilder);
	}

	const FSlateElementsBuffers SlateElementsBuffers = BuildSlateElementsBuffers(GraphBuilder, BatchData);
	const FSlateElementsBuffers SlateElementsBuffersHDR = BuildSlateElementsBuffers(GraphBuilder, BatchDataHDR);

	FRDGTexture* SlateStencilTexture = nullptr;

	if (BatchData.IsStencilClippingRequired() || BatchDataHDR.IsStencilClippingRequired())
	{
		SlateStencilTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(OutputExtent, PF_DepthStencil, FClearValueBinding::DepthZero, GetSlateTransientDepthStencilFlags()), TEXT("SlateDepthStencil"));
	}
	
	FSlateDrawElementsPassInputs DrawElementsInputs =
	{
		  .SceneViewportTexture         = OutputTexture
		, .ElementsMatrix               = FMatrix44f(ViewportInfo.ProjectionMatrix)
		, .SceneViewRect                = Inputs.SceneViewRect
		, .CursorPosition               = Inputs.CursorPosition
		, .Time                         = Inputs.Time
		, .HDRDisplayColorGamut         = ViewportInfo.HDRMetaData.DisplayColorGamut
		, .HDRPaperWhiteInNits          = ViewportInfo.HDRMetaData.HDRPaperWhiteInNits
		, .UsedSlatePostBuffers         = Inputs.UsedSlatePostBuffers
		, .ViewportScaleUI              = Inputs.ViewportScaleUI
		, .bWireframe                   = GSlateWireframe
		, .bElementsTextureIsHDRDisplay = ViewportInfo.HDRMetaData.bHDRSupported
	};
	
	if (ScreenshotState.ViewportToCapture == &ViewportInfo)
	{
		// we are in a screenshot, render everything to the base draw layer
		LayerRange = TInterval<int32>(std::numeric_limits<int32>::min(), std::numeric_limits<int32>::max());
	}

	if (bCompositeUIWithSceneHDR)
	{
		// Color deficiency correction is performed inside of the CompositeUI pass instead.
		DrawElementsInputs.bAllowColorDeficiencyCorrection = false;

		if (!BatchDataHDR.GetRenderBatches().IsEmpty())
		{
			DrawElementsInputs.ElementsTexture = OutputTexture;
			DrawElementsInputs.ElementsLoadAction = ERenderTargetLoadAction::EClear;
			DrawElementsInputs.ElementsBuffers = SlateElementsBuffersHDR;
			DrawElementsInputs.StencilTexture = BatchDataHDR.IsStencilClippingRequired() ? SlateStencilTexture : nullptr;

			AddSlateDrawElementsPass(GraphBuilder, *RenderingPolicy, DrawElementsInputs, 
									 BatchDataHDR.GetRenderBatches(), BatchDataHDR.GetFirstRenderBatchIndex(), 
									 LayerRange);
		}

		DrawElementsInputs.bElementsTextureIsHDRDisplay = false;
	}

	DrawElementsInputs.ElementsTexture = ElementsTexture;
	DrawElementsInputs.ElementsLoadAction = bClearElementsTexture ? ERenderTargetLoadAction::EClear : ERenderTargetLoadAction::ELoad;
	DrawElementsInputs.ElementsBuffers = SlateElementsBuffers;
	DrawElementsInputs.StencilTexture = BatchData.IsStencilClippingRequired() ? SlateStencilTexture : nullptr;
	
	AddSlateDrawElementsPass(GraphBuilder, *RenderingPolicy, DrawElementsInputs, 
							 BatchData.GetRenderBatches(), BatchData.GetFirstRenderBatchIndex(),
							 LayerRange);

	if (bCompositeUIWithSceneHDR)
	{
		RDG_EVENT_SCOPE(GraphBuilder, "CompositeUI");

		FRDGTexture* ElementsWriteMaskTexture = nullptr;

		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

		if (RHISupportsRenderTargetWriteMask(GMaxRHIShaderPlatform))
		{
			FRenderTargetWriteMask::Decode(GraphBuilder, ShaderMap, MakeArrayView({ ElementsTexture }), ElementsWriteMaskTexture, ETextureCreateFlags::None, TEXT("ElementsWriteMaskTexture"));
		}

		static const auto CVarOutputDevice = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.HDR.Display.OutputDevice"));

		FCompositeShader::FPermutationDomain PermutationVector;
		PermutationVector.Set<FCompositeShader::FSCRGBEncoding>(ViewportInfo.HDRMetaData.DisplayOutputFormat == EDisplayOutputFormat::HDR_ACES_1000nit_ScRGB || ViewportInfo.HDRMetaData.DisplayOutputFormat == EDisplayOutputFormat::HDR_ACES_2000nit_ScRGB);
		PermutationVector.Set<FCompositeShader::FApplyColorDeficiency>(GSlateColorDeficiencyType != EColorVisionDeficiency::NormalVision && GSlateColorDeficiencySeverity > 0);

		FCompositeShaderCommonParameters CommonParameters;
		CommonParameters.UIWriteMaskTexture = ElementsWriteMaskTexture;
		CommonParameters.UITexture = ElementsTexture;
		CommonParameters.UISampler = TStaticSamplerState<SF_Point>::GetRHI();
		CommonParameters.UILevel = CVarUILevel.GetValueOnRenderThread();
		CommonParameters.UILuminance = CVarHDRUILuminanceMode.GetValueOnRenderThread()
				? CVarHDRUILuminance.GetValueOnRenderThread()
				: ViewportInfo.HDRMetaData.HDRPaperWhiteInNits;
		CommonParameters.UICompositeEOTF = CVarHDRUICompositeEOTF.GetValueOnRenderThread();
		CommonParameters.UICompositeEOTFGamma = CVarHDRUICompositeEOTFGamma.GetValueOnRenderThread();
		CommonParameters.UICompositeBlend = CVarHDRUICompositeBlend.GetValueOnRenderThread();
		const float UICompositeBlendGamma = CVarHDRUICompositeBlendGamma.GetValueOnRenderThread();
		CommonParameters.UICompositeBlendGamma = UICompositeBlendGamma;
		CommonParameters.UICompositeBlendInvGamma = 1.f / UICompositeBlendGamma;
		CommonParameters.ColorVisionDeficiencySeverity = (float)GSlateColorDeficiencySeverity;
		CommonParameters.ColorVisionDeficiencyType = (float)GSlateColorDeficiencyType;
		CommonParameters.bSimulateCorrectionWithDeficiency = GSlateShowColorDeficiencyCorrectionWithDeficiency ? 1.0f : 0.0f;
		CommonParameters.bCorrectDeficiency = GSlateColorDeficiencyCorrection ? 1.0f : 0.0f;

		if (FCompositeCS::IsShaderSupported(GMaxRHIShaderPlatform))
		{
			FCompositeCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCompositeCS::FParameters>();
			PassParameters->Common = CommonParameters;
			PassParameters->RWSceneTexture = GraphBuilder.CreateUAV(OutputTexture);
			PassParameters->SceneTextureDimensions = FVector4f((float)OutputExtent.X, (float)OutputExtent.Y, 1.0f/(float)OutputExtent.X, 1.0f/(float)OutputExtent.Y);

			TShaderMapRef<FCompositeCS> ComputeShader(ShaderMap, PermutationVector);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("CompositeUI"),
				ComputeShader,
				PassParameters,
				FIntVector(
					FMath::DivideAndRoundUp<int32>(OutputExtent.X, FCompositeCS::NUM_THREADS_PER_GROUP),
					FMath::DivideAndRoundUp<int32>(OutputExtent.Y, FCompositeCS::NUM_THREADS_PER_GROUP),
					1));
		}
		else
		{
			FRDGTexture* ViewportCopyTexture = GraphBuilder.CreateTexture(
				FRDGTextureDesc::Create2D(
					OutputExtent,
					OutputTexture->Desc.Format,
					FClearValueBinding::Transparent,
					GetSlateTransientRenderTargetFlags()),
				TEXT("SlateViewportCopyTexture"));

			AddCopyTexturePass(GraphBuilder, OutputTexture, ViewportCopyTexture);
				
			const FScreenPassTextureViewport Viewport(OutputTexture);

			FCompositePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCompositePS::FParameters>();
			PassParameters->Common = CommonParameters;
			PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputTexture, ERenderTargetLoadAction::ENoAction);
			PassParameters->SceneTexture = ViewportCopyTexture;
			PassParameters->SceneSampler = TStaticSamplerState<SF_Point>::GetRHI();

			TShaderMapRef<FCompositePS> PixelShader(ShaderMap, PermutationVector);

			AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("CompositeUI"), FScreenPassViewInfo(), Viewport, Viewport, PixelShader, PassParameters);
		}
	}

	if (bCompositeStereoToSwapChain)
	{
		FRDGTexture* SwapChainTexture = RegisterExternalTexture(GraphBuilder, SwapChainTextureRHI, TEXT("StereoSpectatorSwapChainTexture"));
		GraphBuilder.SetTextureAccessFinal(SwapChainTexture, ERHIAccess::Present);
		GEngine->StereoRenderingDevice->RenderTexture_RenderThread(GraphBuilder, SwapChainTexture, OutputTexture,
																   Inputs.WindowElementList->GetWindowSize());
	}
	else
	{
		GraphBuilder.SetTextureAccessFinal(OutputTexture, ERHIAccess::Present);
	}

	if (bRequiresVirtualTextureFeedback)
	{
		VirtualTexture::EndFeedback(GraphBuilder);
	}

	if (ScreenshotState.ViewportToCapture == &ViewportInfo)
	{
		// Sanity check to make sure the user specified a valid screenshot rect.
		FIntRect ViewRectClamped;
		ViewRectClamped.Min = ScreenshotState.ViewRect.Min;
		ViewRectClamped.Max = ScreenshotState.ViewRect.Max.ComponentMin(OutputExtent);
		ViewRectClamped.Max = ScreenshotState.ViewRect.Min.ComponentMax(ViewRectClamped.Max);

		if (ViewRectClamped != ScreenshotState.ViewRect)
		{
			UE_LOGF(LogSlate, Warning, "Slate: Screenshot rect max coordinate had to be clamped from [%d, %d] to [%d, %d]", ScreenshotState.ViewRect.Max.X, ScreenshotState.ViewRect.Max.Y, ViewRectClamped.Max.X, ViewRectClamped.Max.Y);
		}

		if (!ViewRectClamped.IsEmpty())
		{
			AddReadbackTexturePass(GraphBuilder, RDG_EVENT_NAME("ScreenshotReadback"), OutputTexture,
				[this, OutputTexture, ViewRectClamped, ColorDataHDR = ScreenshotState.ColorDataHDR, ColorData = ScreenshotState.ColorData] (FRHICommandListImmediate& RHICmdList)
			{
				if (ColorDataHDR)
				{
					RHICmdList.ReadSurfaceData(OutputTexture->GetRHI(), ViewRectClamped, *ColorDataHDR, FReadSurfaceDataFlags(RCM_MinMax));
				}
				else
				{
					check(ColorData);
					RHICmdList.ReadSurfaceData(OutputTexture->GetRHI(), ViewRectClamped, *ColorData, FReadSurfaceDataFlags());
				}
			});
		}
	}
}

FSlateDrawWindowPassOutputs FSlateRHIRenderer::DrawWindow_RenderThread(FRDGBuilder& GraphBuilder, const FSlateDrawWindowPassInputs& Inputs)
{
	LLM_SCOPE(ELLMTag::SceneRender);
	
	FSlateViewportInfo& ViewportInfo = *Inputs.ViewportInfo;
	
	FMaterialRenderProxy::UpdateDeferredCachedUniformExpressions(GraphBuilder.RHICmdList);
	GetRendererModule().InitializeSystemTextures(GraphBuilder.RHICmdList);
	
	TOptional<FSlateDrawWindowPassOutputs> Outputs;
	
	uint32 GPUIndex = ViewportInfo.IsViewportRHI()
		? RHIGetViewportNextPresentGPUIndex(ViewportInfo.GetViewportRHI())
		: 0;

	RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::FromIndex(GPUIndex));
#if WANTS_DRAW_MESH_EVENTS
	RDG_EVENT_SCOPE_CONDITIONAL_STAT(GraphBuilder,  Inputs.WindowTitle.IsEmpty(), SlateUI, "SlateUI Title = <none>");
	RDG_EVENT_SCOPE_CONDITIONAL_STAT(GraphBuilder, !Inputs.WindowTitle.IsEmpty(), SlateUI, "SlateUI Title = %s", *Inputs.WindowTitle);
#else
	RDG_EVENT_SCOPE_STAT(GraphBuilder, SlateUI, "SlateUI");
#endif
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, Slate);
	TRACE_CPUPROFILER_EVENT_SCOPE(Slate::DrawWindow_RenderThread);

	for (TInterval<int32> const& LayerRange : ViewportInfo.Layers.Ranges)
	{
		if (ViewportInfo.IsViewportRHI())
		{
			// @todo refactor this - remove SetDefaultNativeLayer
			ViewportInfo.GetViewportRHI()->SetDefaultNativeLayer(LayerRange.Min);
		}
			
		FRHITexture* ViewportTextureRHI = nullptr;
		FRHITexture* OutputTextureRHI = nullptr;

		DrawWindowViewport_RenderThread(
			  Inputs.Window->GetViewport().Get()
			, GraphBuilder
			, Inputs
			, &ViewportTextureRHI
			, &OutputTextureRHI
			, ViewportInfo
			, Inputs.WindowElementList->GetBatchData()
			, Inputs.WindowElementList->GetBatchDataHDR()
			, LayerRange.Min
			, LayerRange
		);
			
		if (!Outputs.IsSet())
		{
			Outputs = FSlateDrawWindowPassOutputs
			{
				.ViewportInfo = ViewportInfo,
				.ViewportTextureRHI = ViewportTextureRHI,
				.OutputTextureRHI = OutputTextureRHI
			};
		}
		else if (ScreenshotState.ViewportToCapture == &ViewportInfo)
		{
			// we are in a screenshot, we just needed to render everything to the base draw layer
			break;
		}
	}

	if (ViewportInfo.IsViewportRHI())
	{
		// @todo refactor this - remove SetDefaultNativeLayer
		ViewportInfo.GetViewportRHI()->SetDefaultNativeLayer(std::numeric_limits<int32>::min());
	}

	return Outputs.GetValue();
}

void FSlateRHIRenderer::PresentWindow_RenderThread(FRHICommandListImmediate& RHICmdList, const FSlateDrawWindowPassInputs& DrawPassInputs, const FSlateDrawWindowPassOutputs& DrawPassOutputs)
{
	OnBackBufferReadyToPresentDelegate.Broadcast(*DrawPassInputs.Window, DrawPassOutputs.ViewportInfo);

	uint32 StartTime = FPlatformTime::Cycles();
	
	if (GRHIGlobals.NeedsExtraTransitions)
	{
		TArray<FRHITransitionInfo, TInlineAllocator<2>> Transitions =
		{
			FRHITransitionInfo(DrawPassOutputs.OutputTextureRHI, ERHIAccess::Unknown, ERHIAccess::Present)
		};

		if (DrawPassOutputs.ViewportInfo.IsViewportRHI())
		{
			if (FRHITexture* OptionalSDRBuffer = DrawPassOutputs.ViewportInfo.GetViewportRHI()->GetOptionalSDRBackBuffer(DrawPassOutputs.OutputTextureRHI))
			{
				RHICmdList.Transition(FRHITransitionInfo(OptionalSDRBuffer, ERHIAccess::Unknown, ERHIAccess::Present));
			}
		}

		RHICmdList.Transition(Transitions);
	}

	if (DrawPassOutputs.ViewportInfo.IsViewportRHI())
	{
		DrawPassOutputs.ViewportInfo.GetViewportRHI()->SetPresentLatencyCallbacks(
			[CurrentFrameCounter = GFrameCounterRenderThread]() { UEngine::SetPresentLatencyMarkerStart(CurrentFrameCounter); },
			[CurrentFrameCounter = GFrameCounterRenderThread]() { UEngine::SetPresentLatencyMarkerEnd  (CurrentFrameCounter); }
		);

		RHICmdList.EndDrawingViewport(DrawPassOutputs.ViewportInfo.GetViewportRHI(), FRHIPresentArgs(/*.FrameCounter*/ GFrameCounterRenderThread, /*.bPresent*/ DrawPassInputs.bPresent, /*.bLockToVsync*/ DrawPassInputs.bLockToVsync));
	}

	uint32 EndTime = FPlatformTime::Cycles();

	GSwapBufferTime = EndTime - StartTime;
	SET_CYCLE_COUNTER(STAT_PresentTime, GSwapBufferTime);

	static uint32 LastTimestamp = FPlatformTime::Cycles();
	uint32 ThreadTime = EndTime - LastTimestamp;
	LastTimestamp = EndTime;

	uint32 RenderThreadIdle = 0;

	UE::Stats::FThreadIdleStats& RenderThread = UE::Stats::FThreadIdleStats::Get();
	GRenderThreadIdle[ERenderThreadIdleTypes::WaitingForAllOtherSleep] = RenderThread.Waits;
	GRenderThreadIdle[ERenderThreadIdleTypes::WaitingForGPUPresent] += GSwapBufferTime;

	SET_CYCLE_COUNTER(STAT_RenderingIdleTime_RenderThreadSleepTime, GRenderThreadIdle[ERenderThreadIdleTypes::WaitingForAllOtherSleep]);
	SET_CYCLE_COUNTER(STAT_RenderingIdleTime_WaitingForGPUQuery   , GRenderThreadIdle[ERenderThreadIdleTypes::WaitingForGPUQuery     ]);
	SET_CYCLE_COUNTER(STAT_RenderingIdleTime_WaitingForGPUPresent , GRenderThreadIdle[ERenderThreadIdleTypes::WaitingForGPUPresent   ]);

	const uint32 RenderThreadNonCriticalWaits = RenderThread.Waits - RenderThread.WaitsCriticalPath;
	const uint32 RenderThreadWaitingForGPUQuery = GRenderThreadIdle[ERenderThreadIdleTypes::WaitingForGPUQuery];

	// Set the RenderThreadIdle CSV stats
	CSV_CUSTOM_STAT(RenderThreadIdle, Total          , FPlatformTime::ToMilliseconds(RenderThread.Waits            ), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(RenderThreadIdle, CriticalPath   , FPlatformTime::ToMilliseconds(RenderThread.WaitsCriticalPath), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(RenderThreadIdle, SwapBuffer     , FPlatformTime::ToMilliseconds(GSwapBufferTime               ), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(RenderThreadIdle, NonCriticalPath, FPlatformTime::ToMilliseconds(RenderThreadNonCriticalWaits  ), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(RenderThreadIdle, GPUQuery       , FPlatformTime::ToMilliseconds(RenderThreadWaitingForGPUQuery), ECsvCustomStatOp::Set);

	for (int32 Index = 0; Index < ERenderThreadIdleTypes::Num; Index++)
	{
		RenderThreadIdle += GRenderThreadIdle[Index];
		GRenderThreadIdle[Index] = 0;
	}

	SET_CYCLE_COUNTER(STAT_RenderingIdleTime, RenderThreadIdle);
	GRenderThreadTime = (ThreadTime > RenderThreadIdle) ? (ThreadTime - RenderThreadIdle) : ThreadTime;
	GRenderThreadWaitTime = RenderThreadIdle;

	// Compute GRenderThreadTimeCriticalPath
	uint32 RenderThreadNonCriticalPathIdle = RenderThreadIdle - RenderThread.WaitsCriticalPath;
	GRenderThreadTimeCriticalPath = (ThreadTime > RenderThreadNonCriticalPathIdle) ? (ThreadTime - RenderThreadNonCriticalPathIdle) : ThreadTime;
	SET_CYCLE_COUNTER(STAT_RenderThreadCriticalPath, GRenderThreadTimeCriticalPath);

	if (CVarRenderThreadTimeIncludesDependentWaits.GetValueOnRenderThread())
	{
		// Optionally force the renderthread stat to include dependent waits
		GRenderThreadTime = GRenderThreadTimeCriticalPath;
	}

	// Reset the idle stats
	RenderThread.Reset();
	
	static TOptional<uint32> RHITCycles;
	if (IsRunningRHIInSeparateThread())
	{
		RHICmdList.EnqueueLambda([](FRHICommandListImmediate&)
		{
			// Update RHI thread time
			UE::Stats::FThreadIdleStats& RHIThreadStats = UE::Stats::FThreadIdleStats::Get();

			if (!RHITCycles.IsSet())
			{
				RHITCycles = FPlatformTime::Cycles();
			}

			uint32 Next = FPlatformTime::Cycles();

			int32 Result = int32(Next - RHITCycles.GetValue() - RHIThreadStats.Waits);
			RHITCycles = Next;

			FPlatformAtomics::AtomicStore((int32*)&GRHIThreadTime, FMath::Max(Result, 0));
			RHIThreadStats.Reset();
		});
	}
	else
	{
		RHITCycles.Reset();
	}
}

void FSlateRHIRenderer::DrawWindows(FSlateDrawBuffer& WindowDrawBuffer)
{
	DrawWindows_Private(WindowDrawBuffer);
}

void FSlateRHIRenderer::PrepareToTakeScreenshot(const FIntRect& Rect, TArray<FColor>* OutColorData, SWindow* InScreenshotWindow)
{
	check(OutColorData);

	ScreenshotState.ViewRect = Rect;
	ScreenshotState.ViewportToCapture = WindowToViewportInfo.FindRef(InScreenshotWindow);
	ScreenshotState.ColorData = OutColorData;
	ScreenshotState.ColorDataHDR = nullptr;
}

void FSlateRHIRenderer::PrepareToTakeHDRScreenshot(const FIntRect& Rect, TArray<FLinearColor>* OutColorData, SWindow* InScreenshotWindow)
{
	check(OutColorData);

	ScreenshotState.ViewRect = Rect;
	ScreenshotState.ViewportToCapture = WindowToViewportInfo.FindRef(InScreenshotWindow);
	ScreenshotState.ColorData = nullptr;
	ScreenshotState.ColorDataHDR = OutColorData;
}

struct FSlateDrawWindowsCommand : public TConcurrentLinearObject<FSlateDrawWindowsCommand>
{
	bool IsEmpty() { return Windows.IsEmpty() && DeferredUpdates.IsEmpty(); }

	TArray<FSlateDrawWindowPassInputs, FConcurrentLinearArrayAllocator> Windows;
	TArray<FRenderThreadUpdateContext, FConcurrentLinearArrayAllocator> DeferredUpdates;
};

#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
void FSlateRHIRenderer::CreateNativeLayer(int32 NewNativeLayer, SWindow& InWindow, void* NativeViewHandle)
{
	FSlateViewportInfo* ViewportInfo = WindowToViewportInfo.FindRef(&InWindow);
	if(ViewportInfo != nullptr)
	{
		ViewportInfo->CreateNativeLayer(NewNativeLayer, NativeViewHandle);
	}
}

void FSlateRHIRenderer::DeleteNativeLayer(int32 OldNativeLayer, SWindow& InWindow)
{
	FSlateViewportInfo* ViewportInfo = WindowToViewportInfo.FindRef(&InWindow);
	if(ViewportInfo != nullptr)
	{
		ViewportInfo->DeleteNativeLayer(OldNativeLayer);
	}
}
#endif

void FSlateRHIRenderer::DrawWindows_RenderThread(FRHICommandListImmediate& RHICmdList, TConstArrayView<FSlateDrawWindowPassInputs> Windows, TConstArrayView<FRenderThreadUpdateContext> DeferredUpdates)
{
	struct FWindowPresentCommand
	{
		FWindowPresentCommand(const FSlateDrawWindowPassInputs& InInputs, const FSlateDrawWindowPassOutputs& InOutputs)
			: Inputs(InInputs)
			, Outputs(InOutputs)
		{}

		const FSlateDrawWindowPassInputs& Inputs;
		FSlateDrawWindowPassOutputs Outputs;
	};

	TArray<FWindowPresentCommand, FConcurrentLinearArrayAllocator> WindowPresentCommands;
	WindowPresentCommands.Reserve(Windows.Num());

	int32 WindowIndex = 0;

	do
	{
		{
			FRDGBuilder GraphBuilder(RHICmdList, RDG_EVENT_NAME("Slate"), ERDGBuilderFlags::ParallelSetup | ERDGBuilderFlags::ParallelExecute);

			for (const FRenderThreadUpdateContext& DeferredUpdateContext : DeferredUpdates)
			{
				DeferredUpdateContext.Renderer->DrawWindowToTarget_RenderThread(GraphBuilder, DeferredUpdateContext);
			}

			// D3D12 can't handle more than 8 swap chains at a time, start a new batch if we hit this amount and continue with a new builder.
			for (int32 NumWindows = 0; NumWindows < 8 && WindowIndex < Windows.Num(); ++NumWindows, ++WindowIndex)
			{
				const FSlateDrawWindowPassInputs& DrawWindowPassInputs = Windows[WindowIndex];
				WindowPresentCommands.Emplace(DrawWindowPassInputs, DrawWindow_RenderThread(GraphBuilder, DrawWindowPassInputs));
			}

			GraphBuilder.AddDispatchHint();
			GraphBuilder.Execute();
		}

		for (const FRenderThreadUpdateContext& DeferredUpdateContext : DeferredUpdates)
		{
			DeferredUpdateContext.Renderer->ReleaseDrawBuffer(*DeferredUpdateContext.WindowDrawBuffer);
		}

		for (const FWindowPresentCommand& Command : WindowPresentCommands)
		{
			PresentWindow_RenderThread(RHICmdList, Command.Inputs, Command.Outputs);
		}
		WindowPresentCommands.Reset();

		DeferredUpdates = {};

	} while (WindowIndex < Windows.Num());
}

void FSlateRHIRenderer::DrawWindows_Private(FSlateDrawBuffer& WindowDrawBuffer)
{
	checkSlow(IsThreadSafeForSlateRendering());
	CSV_SCOPED_TIMING_STAT(Slate, DrawWindows_Private);

	if (bUpdateHDRDisplayInformation && IsHDRAllowed() && IsInGameThread())
	{
		FlushRenderingCommands();
		RHIHandleDisplayChange();
		bUpdateHDRDisplayInformation = false;
	}

	if (DoesThreadOwnSlateRendering())
	{
		ResourceManager->UpdateTextureAtlases();
	}

	USlateRHIRendererSettings* RendererSettings = USlateRHIRendererSettings::GetMutable();

	const float AppDeltaTime = FApp::GetDeltaTime();
	const FGameTime AppDilatedTime = FGameTime::CreateDilated(FPlatformTime::Seconds() - GStartTime, AppDeltaTime, FApp::GetCurrentTime() - GStartTime, AppDeltaTime);
	const FVector2f AppCursorPosition = FSlateApplication::Get().GetCursorPos();
	const bool bAppCanRender = GIsClient && !IsRunningCommandlet() && !GUsingNullRHI;
	const bool bAppCanRenderPostProcess = RendererSettings && IsInGameThread() && UAssetManager::IsInitialized() && bAppCanRender && CVarCopyBackbufferToSlatePostRenderTargets.GetValueOnGameThread() > 0;
	EPixelFormat AppViewportSceneFormat = PF_Unknown;
	FIntPoint AppViewportExtentMax = FIntPoint::ZeroValue;
	ESlatePostRT PostProcessAnyUsedBits = ESlatePostRT::None;

	const TSharedRef<FSlateFontCache> FontCache = SlateFontServices->GetFontCache();

	struct FWindowToRender
	{
		SWindow* Window;
		FSlateWindowElementList* WindowElementList;
		FSlateViewportInfo* ViewportInfo;
		FIntPoint ViewportOffset;
		FIntPoint ViewportExtent;
		FIntRect  ViewportRect;
		float ViewportScaleUI;
		ESlatePostRT PostProcessUsedBits;
		ESlatePostRT PostProcessCustumDrawBits;
		FIntPoint CursorPosition;
		bool bLockToVsync;
		bool bPresent;
	};

	TArray<FWindowToRender, FConcurrentLinearArrayAllocator> WindowsToRender;

	if (bAppCanRender)
	{
		WindowsToRender.Reserve(WindowDrawBuffer.GetWindowElementLists().Num());

		for (const TSharedRef<FSlateWindowElementList>& WindowElementListRef : WindowDrawBuffer.GetWindowElementLists())
		{
			FSlateWindowElementList* WindowElementList = &(*WindowElementListRef);
			
			SWindow* Window = WindowElementList->GetRenderWindow();

			if (!Window)
			{
				ensureMsgf(false, TEXT("Window isn't valid but being drawn!"));
				continue;
			}

			// This will return zero if both the viewport and the window are zero sized.
			const FVector2f WindowSize = Window->GetViewportSize();
			if (WindowSize.X <= 0.0f || WindowSize.Y <= 0.0f)
			{
				continue;
			}

			TRACE_CPUPROFILER_EVENT_SCOPE(GatherWindowElements);

			// It's possible for a window to not have a viewport, in which case the viewport dimensions will be zero.
			FVector2f ViewportCursorPosition = AppCursorPosition - Window->GetPositionInScreen();
			FIntPoint ViewportOffset = FIntPoint::ZeroValue;
			FIntPoint ViewportExtent = FIntPoint::ZeroValue;
			FIntRect ViewportRect;
			float ViewportScaleUI = Window->GetViewportScaleUIOverride();
			bool bCameraCut = false;

			if (ISlateViewport* Viewport = Window->GetViewport().Get())
			{
				TSharedPtr<SWidget> ViewportWidget = Viewport->GetWidget().Pin();

				if (ViewportWidget)
				{
					ViewportOffset = FIntPoint(
						FMath::RoundToInt32(ViewportWidget->GetTickSpaceGeometry().GetAbsolutePosition().X - Window->GetPositionInScreen().X),
						FMath::RoundToInt32(ViewportWidget->GetTickSpaceGeometry().GetAbsolutePosition().Y - Window->GetPositionInScreen().Y));

					ViewportCursorPosition -= ViewportWidget->GetPaintSpaceGeometry().AbsolutePosition;
				}

				ViewportExtent = Viewport->GetSize();
				ViewportRect = FIntRect(ViewportOffset, ViewportOffset + ViewportExtent);

				ensureMsgf(AppViewportSceneFormat == PF_Unknown || AppViewportSceneFormat == Viewport->GetSceneTargetFormat(),
					TEXT("Multiple viewport formats coming from multiple windows are not a supported scenario in slate. This will cause undefined behavior with Slate Post Buffers."));

				AppViewportSceneFormat = Viewport->GetSceneTargetFormat();
				AppViewportExtentMax = AppViewportExtentMax.ComponentMax(ViewportExtent);

				if (ViewportScaleUI < 0.0f)
				{
					ViewportScaleUI = GetDefault<UUserInterfaceSettings>()->GetDPIScaleBasedOnSize(ViewportExtent);
				}

				bCameraCut = Viewport->IsCameraCut();
			}

			if (FSlateViewportInfo* ViewportInfo = WindowToViewportInfo.FindRef(Window))
			{
				if (Window->IsViewportSizeDrivenByWindow())
				{
					ViewportInfo->ResizeViewport(ViewportInfo->Extent, IsViewportFullscreen(*Window));
				}

				Window->SetIsHDR(ViewportInfo->HDRMetaData.bHDRSupported);
				Window->ResetViewportScaleUIOverride();

				ElementBatcher->SetCompositeHDRViewports(ViewportInfo->HDRMetaData.bHDRSupported && CompositeUIWithSceneHDR());
				ElementBatcher->AddElements(*WindowElementList);

				const bool bWindowCanRenderPostProcess       = bAppCanRender && ViewportExtent != FIntPoint::ZeroValue;
				const ESlatePostRT PostProcessUsedBits       = bWindowCanRenderPostProcess ? ElementBatcher->GetUsedSlatePostBuffers() : ESlatePostRT::None;
				const ESlatePostRT PostProcessCustomDrawBits = bWindowCanRenderPostProcess ? ElementBatcher->GetResourceUpdatingPostBuffers() : ESlatePostRT::None;
				const bool bLockToVsync                      = IsVSyncRequired(*ElementBatcher);
				const bool bPresent                          = ShouldPresent(bCameraCut);

				ElementBatcher->ResetBatches();
				ElementBatcher->SetCompositeHDRViewports(false);

				WindowsToRender.Emplace(FWindowToRender
				{
					  .Window                    = Window
					, .WindowElementList         = WindowElementList
					, .ViewportInfo              = ViewportInfo
					, .ViewportOffset            = ViewportOffset
					, .ViewportExtent            = ViewportExtent
					, .ViewportRect              = ViewportRect
					, .ViewportScaleUI           = ViewportScaleUI
					, .PostProcessUsedBits       = PostProcessUsedBits
					, .PostProcessCustumDrawBits = PostProcessCustomDrawBits
					, .CursorPosition            = FIntPoint(ViewportCursorPosition.X, ViewportCursorPosition.Y)
					, .bLockToVsync              = bLockToVsync
					, .bPresent					 = bPresent
				});

				PostProcessAnyUsedBits |= PostProcessUsedBits;
			}
		}
	}

	// Update the font cache now that all element batches were processed.
	FontCache->UpdateCache();

	// Allocate any post process render targets that are used by any viewport.
	if (bAppCanRenderPostProcess)
	{
		if (AppViewportExtentMax.X != 0 && AppViewportExtentMax.Y != 0)
		{
			for (ESlatePostRT Bit : MakeFlagsRange(PostProcessAnyUsedBits))
			{
				if (UTextureRenderTarget2D* RenderTarget = RendererSettings->LoadGetPostBufferRT(Bit))
				{
					const int32 PostBufferDownscale = RendererSettings->GetSlatePostBufferDownscaleFactor(Bit);
					FIntPoint TargetSize = AppViewportExtentMax;
					if (PostBufferDownscale > 1)
					{
						TargetSize = GetDownscaledExtent(TargetSize, PostBufferDownscale);
					}

					if (RenderTarget->SizeX != TargetSize.X || RenderTarget->SizeY != TargetSize.Y || RenderTarget->GetFormat() != AppViewportSceneFormat)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(AllocatePostProcessTexture);
						RenderTarget->InitCustomFormat(TargetSize.X, TargetSize.Y, AppViewportSceneFormat, true);
					}
					PostProcessRenderTargets.LastUsedFrameCounter[(int32)Bit] = GFrameCounter;
				}
			}
		}

		for (ESlatePostRT Bit : MakeFlagsRange(ESlatePostRT::All & ~PostProcessAnyUsedBits))
		{
			UTextureRenderTarget2D* RenderTarget = RendererSettings->TryGetPostBufferRT(Bit);;

			if (RenderTarget
				&& RenderTarget->GetResource()
				&& RenderTarget->SizeX != 1
				&& RenderTarget->SizeY != 1
				&& PostProcessRenderTargets.LastUsedFrameCounter[(int32)Bit] < GFrameCounter)
			{
				// Trim unused post process render targets down to 1x1 to reclaim memory.
				TRACE_CPUPROFILER_EVENT_SCOPE(TrimPostProcessTexture);
				RenderTarget->InitCustomFormat(1, 1, PF_A2B10G10R10, true);
			}
		}
	}

	TUniquePtr<FSlateDrawWindowsCommand> DrawWindowsCommand = MakeUnique<FSlateDrawWindowsCommand>();
	DrawWindowsCommand->Windows.Reserve(WindowsToRender.Num());
	DrawWindowsCommand->DeferredUpdates = MoveTemp(DeferredUpdateContexts);

	bool bScreenshotProcessed = false;

	for (const FWindowToRender& WindowToRender : WindowsToRender)
	{
		bScreenshotProcessed |= ScreenshotState.ViewportToCapture == WindowToRender.ViewportInfo;

		if (bAppCanRender)
		{
			DrawWindowsCommand->Windows.Emplace(FSlateDrawWindowPassInputs
			{
				  .Renderer                  = this
				, .WindowElementList         = WindowToRender.WindowElementList
				, .Window                    = WindowToRender.Window
				, .ViewportInfo              = WindowToRender.ViewportInfo
				, .CursorPosition            = WindowToRender.CursorPosition
				, .SceneViewRect             = WindowToRender.ViewportRect
				, .ViewportScaleUI           = WindowToRender.ViewportScaleUI
				, .UsedSlatePostBuffers      = WindowToRender.PostProcessUsedBits
			#if WANTS_DRAW_MESH_EVENTS
				, .WindowTitle               = WindowToRender.Window->GetTitle().ToString()
			#endif
				, .Time                      = AppDilatedTime
				, .bLockToVsync              = WindowToRender.bLockToVsync
			#if ALPHA_BLENDED_WINDOWS
				, .bClear                    = WindowToRender.Window->GetTransparencySupport() == EWindowTransparency::PerPixel
			#endif
				, .bPresent                  = WindowToRender.bPresent
			});
		}
	}

	if (!DrawWindowsCommand->IsEmpty())
	{
		ENQUEUE_RENDER_COMMAND(SlateDrawWindowsCommand)([this, DrawWindowsCommand = MoveTemp(DrawWindowsCommand)](FRHICommandListImmediate& RHICmdList)
		{
			DrawWindows_RenderThread(RHICmdList, DrawWindowsCommand->Windows, DrawWindowsCommand->DeferredUpdates);
		});

		check(DeferredUpdateContexts.IsEmpty());
	}

	if (bScreenshotProcessed)
	{
		FlushRenderingCommands();
		ScreenshotState = {};
	}

	for (const FWindowToRender& WindowToRender : WindowsToRender)
	{
		SlateWindowRendered.Broadcast(*WindowToRender.Window);
	}

	FlushPendingDeletes();
	FontCache->ConditionalFlushCache();
	ResourceManager->ConditionalFlushAtlases();
}

FIntPoint FSlateRHIRenderer::GenerateDynamicImageResource(const FName InTextureName)
{
	check(IsInGameThread());

	uint32 Width = 0;
	uint32 Height = 0;
	TArray<uint8> RawData;

	TSharedPtr<FSlateDynamicTextureResource> TextureResource = ResourceManager->GetDynamicTextureResourceByName(InTextureName);
	if (!TextureResource.IsValid())
	{
		// Load the image from disk
		bool bSucceeded = ResourceManager->LoadTexture(InTextureName, InTextureName.ToString(), Width, Height, RawData);
		if (bSucceeded)
		{
			TextureResource = ResourceManager->MakeDynamicTextureResource(InTextureName, Width, Height, RawData);
		}
	}

	return TextureResource.IsValid() ? TextureResource->Proxy->ActualSize : FIntPoint(0, 0);
}

bool FSlateRHIRenderer::GenerateDynamicImageResource(FName ResourceName, uint32 Width, uint32 Height, const TArray< uint8 >& Bytes)
{
	check(IsInGameThread());

	TSharedPtr<FSlateDynamicTextureResource> TextureResource = ResourceManager->GetDynamicTextureResourceByName(ResourceName);
	if (!TextureResource.IsValid())
	{
		TextureResource = ResourceManager->MakeDynamicTextureResource(ResourceName, Width, Height, Bytes);
	}
	return TextureResource.IsValid();
}

bool FSlateRHIRenderer::GenerateDynamicImageResource(FName ResourceName, FSlateTextureDataRef TextureData)
{
	check(IsInGameThread());

	TSharedPtr<FSlateDynamicTextureResource> TextureResource = ResourceManager->GetDynamicTextureResourceByName(ResourceName);
	if (!TextureResource.IsValid())
	{
		TextureResource = ResourceManager->MakeDynamicTextureResource(ResourceName, TextureData);
	}
	return TextureResource.IsValid();
}

FSlateResourceHandle FSlateRHIRenderer::GetResourceHandle(const FSlateBrush& Brush, FVector2f LocalSize, float DrawScale)
{
	return ResourceManager->GetResourceHandle(Brush, LocalSize, DrawScale);
}

bool FSlateRHIRenderer::CanRenderResource(UObject& InResourceObject) const
{
	return Cast<UTexture>(&InResourceObject) || Cast<ISlateTextureAtlasInterface>(&InResourceObject) || Cast<UMaterialInterface>(&InResourceObject);
}

void FSlateRHIRenderer::RemoveDynamicBrushResource( TSharedPtr<FSlateDynamicImageBrush> BrushToRemove )
{
	if (BrushToRemove.IsValid())
	{
		DynamicBrushesToRemove[FreeBufferIndex].Add(BrushToRemove);
	}
}

void FSlateRHIRenderer::FlushCommands() const
{
	if (IsInGameThread() || IsInSlateThread())
	{
		FlushRenderingCommands();
	}
}

void FSlateRHIRenderer::Sync() const
{
	FFrameEndSync::Sync(FFrameEndSync::EFlushMode::EndFrame);
}

void FSlateRHIRenderer::EndFrame() const
{
	ENQUEUE_RENDER_COMMAND(SlateRHIEndFrame)([](FRHICommandListImmediate& RHICmdList)
	{
		RHICmdList.EndFrame();
	});
}

void FSlateRHIRenderer::ReloadTextureResources()
{
	ResourceManager->ReloadTextures();
}

void FSlateRHIRenderer::LoadUsedTextures()
{
	if (ResourceManager.IsValid())
	{
		ResourceManager->LoadUsedTextures();
	}
}

void FSlateRHIRenderer::LoadStyleResources(const ISlateStyle& Style)
{
	if (ResourceManager.IsValid())
	{
		ResourceManager->LoadStyleResources(Style);
	}
}

void FSlateRHIRenderer::ReleaseDynamicResource(const FSlateBrush& InBrush)
{
	ensure(IsInGameThread());
	ResourceManager->ReleaseDynamicResource(InBrush);
}

void* FSlateRHIRenderer::GetViewportResource(const SWindow& Window)
{
	checkSlow(IsThreadSafeForSlateRendering());

	FSlateViewportInfo** InfoPtr = WindowToViewportInfo.Find(&Window);

	if (InfoPtr && (*InfoPtr)->IsViewportRHI())
	{
		return (*InfoPtr)->GetViewportRHI();
	}

	return nullptr;
}

ISlateViewportProvider* FSlateRHIRenderer::GetViewportProvider(const SWindow& Window)
{
	checkSlow(IsThreadSafeForSlateRendering());

	FSlateViewportInfo** InfoPtr = WindowToViewportInfo.Find(&Window);

	if (InfoPtr)
	{
		return *InfoPtr;
	}

	return nullptr;
}

void FSlateRHIRenderer::SetColorVisionDeficiencyType(EColorVisionDeficiency Type, int32 Severity, bool bCorrectDeficiency, bool bShowCorrectionWithDeficiency)
{
	GSlateColorDeficiencyType = Type;
	GSlateColorDeficiencySeverity = FMath::Clamp(Severity, 0, 10);
	GSlateColorDeficiencyCorrection = bCorrectDeficiency;
	GSlateShowColorDeficiencyCorrectionWithDeficiency = bShowCorrectionWithDeficiency;
}

FSlateUpdatableTexture* FSlateRHIRenderer::CreateUpdatableTexture(uint32 Width, uint32 Height)
{
	const bool bCreateEmptyTexture = true;
	FSlateTexture2DRHIRef* NewTexture = new FSlateTexture2DRHIRef(Width, Height, GetSlateRecommendedColorFormat(), nullptr, TexCreate_None, bCreateEmptyTexture);
	BeginInitResource(NewTexture);
	return NewTexture;
}

FSlateUpdatableTexture* FSlateRHIRenderer::CreateSharedHandleTexture(void* SharedHandle)
{
	return nullptr;
}

void FSlateRHIRenderer::ReleaseUpdatableTexture(FSlateUpdatableTexture* Texture)
{
	if (IsInRenderingThread())
	{
		Texture->GetRenderResource()->ReleaseResource();
		delete Texture;
	}
	else
	{
		Texture->Cleanup();
	}
}

ISlateAtlasProvider* FSlateRHIRenderer::GetTextureAtlasProvider()
{
	if (ResourceManager.IsValid())
	{
		return ResourceManager->GetTextureAtlasProvider();
	}

	return nullptr;
}

void FSlateRHIRenderer::RegisterCurrentScene_Impl(FSceneInterface* Scene)
{
	check(IsInGameThread());
	if (Scene && Scene->GetWorld())
	{
		CurrentSceneIndex = ActiveScenes.IndexOfByPredicate([&Scene](const FSceneInterface* TestScene) { return TestScene->GetWorld() == Scene->GetWorld(); });

		if (CurrentSceneIndex == INDEX_NONE)
		{
				CurrentSceneIndex = ActiveScenes.Add(Scene);

			if (CurrentSceneIndex >= 0)
			{
				ENQUEUE_RENDER_COMMAND(RegisterCurrentSceneOnPolicy)([RenderingPolicy = RenderingPolicy.Get(), Scene, CurrentSceneIndex = CurrentSceneIndex](FRHICommandListBase&)
				{
					RenderingPolicy->AddSceneAt(Scene, CurrentSceneIndex);
				});
			}
		}
	}
	else
	{
		CurrentSceneIndex = -1;
	}
}

void FSlateRHIRenderer::OnSceneRemoved(FSceneInterface* Scene)
{
	check(IsInGameThread());
	const int32 SceneIdx = ActiveScenes.Find(Scene);
	if (SceneIdx != INDEX_NONE)
	{
		// Instead of removing this individual scene, forget all of them and force widgets to re-register by invalidating everything.  This way we
		// don't need to rely on the scene destroyer to know (and remember to invalidate) all SViewports that have captured this scene's index, and
		// don't need to implement hole-filling behavior on scene registration to maintain indices.
		FSlateApplication::Get().InvalidateAllWidgets(false);
		ClearScenes();
		// This is sufficient and safe, as long as we can continue to assume that this function is called by the main engine loop outside of any call
		// to FSlateApplication::PrivateDrawWindows (i.e. nobody decides to destroy a game instance from SWidget::Paint).  The danger otherwise is
		// that a persistent widget state or on-the-stack widget rendering routine would use a scene index that we have just destroyed.
	}
}

int32 FSlateRHIRenderer::GetCurrentSceneIndex() const
{
	return CurrentSceneIndex;
}

void FSlateRHIRenderer::SetCurrentSceneIndex(int32 InIndex)
{
#if DO_CHECK
	if (!ensureMsgf(InIndex < 0 || ActiveScenes.Num() > InIndex, TEXT("FSlateRHIRenderer::SetCurrentSceneIndex: Invalid index %d (only %d scenes exist)."), InIndex, ActiveScenes.Num()))
	{
		return;
	}
#endif
	CurrentSceneIndex = InIndex;
}

void FSlateRHIRenderer::ClearScenes()
{
	if (!IsInSlateThread())
	{
		CurrentSceneIndex = -1;
		ActiveScenes.Empty();

		ENQUEUE_RENDER_COMMAND(ClearScenesOnPolicy)([RenderingPolicy = RenderingPolicy.Get()](FRHICommandListBase&)
		{
			RenderingPolicy->ClearScenes();
		});
	}
}

EPixelFormat FSlateRHIRenderer::GetSlateRecommendedColorFormat()
{
	return bIsStandaloneStereoOnlyDevice ? PF_R8G8B8A8 : PF_B8G8R8A8;
}

void FSlateRHIRenderer::DestroyCachedFastPathRenderingData(FSlateCachedFastPathRenderingData* CachedFastPathRenderingData)
{
	check(CachedFastPathRenderingData);
	FScopeLock ScopeLock(GetResourceCriticalSection());
	PendingDeletes.CachedFastPathRenderingData.Emplace(CachedFastPathRenderingData);
}

void FSlateRHIRenderer::DestroyCachedFastPathElementData(FSlateCachedElementData* CachedElementData)
{
	check(CachedElementData);
	FScopeLock ScopeLock(GetResourceCriticalSection());
	PendingDeletes.CachedElementData.Emplace(CachedElementData);
}

void FSlateRHIRenderer::FlushPendingDeletes()
{
	FScopeLock ScopeLock(GetResourceCriticalSection());
	if (!PendingDeletes.IsEmpty())
	{
		ENQUEUE_RENDER_COMMAND(SlateDeferredDelete)([InPendingDeletes = MoveTemp(PendingDeletes)](FRHICommandListBase&)
		{
			for (FSlateCachedFastPathRenderingData* Data : InPendingDeletes.CachedFastPathRenderingData)
			{
				delete Data;
			}

			for (FSlateCachedElementData* Data : InPendingDeletes.CachedElementData)
			{
				delete Data;
			}
		});
		PendingDeletes = {};
	}
}

bool FSlateRHIRenderer::AreShadersInitialized() const
{
#if WITH_EDITORONLY_DATA
	static bool bSlateShadersInitialized = false;
	static FDelegateHandle GlobalShaderCompilationDelegateHandle;

	if (!bSlateShadersInitialized)
	{
		bSlateShadersInitialized = IsGlobalShaderMapComplete(TEXT("SlateElement"));

		// if shaders are initialized, cache the value until global shaders gets recompiled.
		if (bSlateShadersInitialized)
		{
			GlobalShaderCompilationDelegateHandle = GetOnGlobalShaderCompilation().AddLambda([]()
			{
				bSlateShadersInitialized = false;
				GetOnGlobalShaderCompilation().Remove(GlobalShaderCompilationDelegateHandle);
			});
		}
	}
	return bSlateShadersInitialized;
#else
	return true;
#endif
}

FCriticalSection* FSlateRHIRenderer::GetResourceCriticalSection()
{
	return ResourceManager->GetResourceCriticalSection();
}

void FSlateRHIRenderer::ReleaseAccessedResources(bool bImmediatelyFlush)
{
	if (bImmediatelyFlush)
	{
		// As explained above, it's only safe to clear the scenes after invalidating all widgets.  We don't need to call invalidate here, though,
		// because it's also only safe to call ReleaseAccessedResources(true) after invalidating all widgets.
		ClearScenes();

		// Increment resource version to allow buffers to shrink or cached structures to clean up.
		ResourceVersion++;
	}
}

void FSlateRHIRenderer::RequestResize(const TSharedPtr<SWindow>& Window, uint32 NewWidth, uint32 NewHeight)
{
	checkSlow(IsThreadSafeForSlateRendering());

	FSlateViewportInfo* ViewInfo = WindowToViewportInfo.FindRef(Window.Get());

	if (ViewInfo)
	{
		ViewInfo->ResizeViewport(FIntPoint(NewWidth, NewHeight), IsViewportFullscreen(*Window));
	}
}

void FSlateRHIRenderer::AddWidgetRendererUpdate(const FRenderThreadUpdateContext& Context, bool bDeferredRenderTargetUpdate)
{
	if (bDeferredRenderTargetUpdate)
	{
		DeferredUpdateContexts.Add(Context);
	}
	else
	{
		ENQUEUE_RENDER_COMMAND(DrawWidgetRendererImmediate)([Context](FRHICommandListImmediate& RHICmdList)
		{
			FRDGBuilder GraphBuilder(RHICmdList, RDG_EVENT_NAME("SlateWidgetRender"), ERDGBuilderFlags::ParallelSetup | ERDGBuilderFlags::ParallelExecute);
			Context.Renderer->DrawWindowToTarget_RenderThread(GraphBuilder, Context);
			GraphBuilder.Execute();
		});
	}
}
