// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Textures/SlateShaderResource.h"
#include "Rendering/DrawElements.h"
#include "RHI.h"
#include "RenderCommandFence.h"
#include "RenderResource.h"
#include "SlateRHIResourceManager.h"
#include "UnrealClient.h"
#include "Rendering/SlateRenderer.h"
#include "Rendering/SlateDrawBuffer.h"
#include "Slate/SlateTextures.h"
#include "Slate/SlateViewportProvider.h"
#include "RendererInterface.h"

class FSlateElementBatcher;
class FSlateRHIRenderingPolicy;
class ISlateStyle;
class SWindow;

extern FMatrix CreateSlateProjectionMatrix(uint32 Width, uint32 Height);

// Number of draw buffers that can be active at any given time
const uint32 NumDrawBuffers = 3;

// Enable to visualize overdraw in Slate
#define WITH_SLATE_VISUALIZERS !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

struct FFastPathRenderingDataCleanUpList;

struct FSlateDrawWindowPassInputs;
struct FSlateDrawWindowPassOutputs;
struct FSlateViewportInfo;
struct FSlatePostProcessUpdateRequest;

/** A Slate rendering implementation for Unreal engine */
class FSlateRHIRenderer final : public FSlateRenderer
{
public:
	FSlateRHIRenderer( TSharedRef<FSlateFontServices> InSlateFontServices, TSharedRef<FSlateRHIResourceManager> InResourceManager );

	/** FSlateRenderer interface */
	virtual bool Initialize() override;
	virtual void Destroy() override;
	virtual FSlateDrawBuffer& AcquireDrawBuffer() override;
	virtual void ReleaseDrawBuffer(FSlateDrawBuffer& InWindowDrawBuffer) override;
	virtual void OnWindowDestroyed( const TSharedRef<SWindow>& InWindow ) override;
	virtual void RequestResize( const TSharedPtr<SWindow>& Window, uint32 NewWidth, uint32 NewHeight ) override;
	virtual void CreateViewport( const TSharedRef<SWindow> Window ) override;
	virtual void UpdateFullscreenState( const TSharedRef<SWindow> Window, uint32 OverrideResX, uint32 OverrideResY ) override;
	virtual void SetSystemResolution(uint32 Width, uint32 Height) override;
	virtual void RestoreSystemResolution(const TSharedRef<SWindow> InWindow) override;
	virtual void DrawWindows( FSlateDrawBuffer& InWindowDrawBuffer ) override;
	virtual void FlushCommands() const override;
	virtual void Sync() const override;
	virtual void ReleaseDynamicResource( const FSlateBrush& InBrush ) override;
	virtual void RemoveDynamicBrushResource( TSharedPtr<FSlateDynamicImageBrush> BrushToRemove ) override;
	virtual FIntPoint GenerateDynamicImageResource(const FName InTextureName) override;
	virtual bool GenerateDynamicImageResource( FName ResourceName, uint32 Width, uint32 Height, const TArray< uint8 >& Bytes ) override;
	virtual bool GenerateDynamicImageResource( FName ResourceName, FSlateTextureDataRef TextureData ) override;
	virtual FSlateResourceHandle GetResourceHandle(const FSlateBrush& Brush, FVector2f LocalSize, float DrawScale) override;
	virtual bool CanRenderResource(UObject& InResourceObject) const override;
	virtual void* GetViewportResource( const SWindow& Window ) override;
	virtual ISlateViewportProvider* GetViewportProvider(const SWindow& Window) override;
	virtual void SetColorVisionDeficiencyType(EColorVisionDeficiency Type, int32 Severity, bool bCorrectDeficiency, bool bShowCorrectionWithDeficiency) override;
	virtual FSlateUpdatableTexture* CreateUpdatableTexture(uint32 Width, uint32 Height) override;
	virtual FSlateUpdatableTexture* CreateSharedHandleTexture(void* SharedHandle) override;
	virtual void ReleaseUpdatableTexture(FSlateUpdatableTexture* Texture) override;
	virtual ISlateAtlasProvider* GetTextureAtlasProvider() override;
	virtual FCriticalSection* GetResourceCriticalSection() override;
	virtual void ReleaseAccessedResources(bool bImmediatelyFlush) override;
	virtual int32 GetCurrentSceneIndex() const override;
	virtual void SetCurrentSceneIndex(int32 InIndex) override;
	virtual void ClearScenes() override;

protected:
	virtual void RegisterCurrentScene_Impl(FSceneInterface* Scene) override;

public:
	EPixelFormat GetSlateRecommendedColorFormat() override;
	virtual void DestroyCachedFastPathRenderingData(struct FSlateCachedFastPathRenderingData* InRenderingData) override;
	virtual void DestroyCachedFastPathElementData(FSlateCachedElementData* InCachedElementData) override;
	virtual void EndFrame() const override;
	virtual void AddWidgetRendererUpdate(const struct FRenderThreadUpdateContext& Context, bool bDeferredRenderTargetUpdate) override;
	
#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
	virtual void CreateNativeLayer(int32 NewNativeLayer, SWindow& InWindow, void* NativeViewHandle) override;
	virtual void DeleteNativeLayer(int32 OldNativeLayer, SWindow& InWindow) override;
#endif

	/**
	 * Reloads texture resources from disk                   
	 */
	virtual void ReloadTextureResources() override;

	virtual void LoadStyleResources( const ISlateStyle& Style ) override;

	/** Returns whether shaders that Slate depends on have been compiled. */
	virtual bool AreShadersInitialized() const override;

	virtual void PrepareToTakeScreenshot(const FIntRect& Rect, TArray<FColor>* OutColorData, SWindow* ScreenshotWindow) override;
	virtual void PrepareToTakeHDRScreenshot(const FIntRect& Rect, TArray<FLinearColor>* OutColorData, SWindow* ScreenshotWindow) override;

private:
	FSlateDrawWindowPassOutputs DrawWindow_RenderThread(FRDGBuilder& GraphBuilder, const FSlateDrawWindowPassInputs& Inputs);
	
	void DrawWindowViewport_RenderThread(ISlateViewport* SlateViewport,
										 FRDGBuilder& GraphBuilder, const FSlateDrawWindowPassInputs& Inputs,
										 FRHITexture** ViewportTextureRHI, FRHITexture** OutputTextureRHI,
										 FSlateViewportInfo& ViewportInfo, FSlateBatchData& BatchData, 
										 FSlateBatchData& BatchDataHDR, TOptional<int32> NativeLayer, TInterval<int32> LayerRange);

	void PresentWindow_RenderThread(FRHICommandListImmediate& RHICmdList, const FSlateDrawWindowPassInputs& DrawPassInputs, const FSlateDrawWindowPassOutputs& DrawPassOutputs);

	void DrawWindows_RenderThread(FRHICommandListImmediate& RHICmdList, TConstArrayView<FSlateDrawWindowPassInputs> Windows, TConstArrayView<FRenderThreadUpdateContext> DeferredUpdates);

	/** Loads all known textures from Slate styles */
	void LoadUsedTextures();

	EPixelFormat GetViewportPixelFormat(const SWindow& Window, bool bDisplayFormatIsHDR);

	/** 
	 * Creates necessary resources to render a window and sends draw commands to the rendering thread
	 *
	 * @param WindowDrawBuffer	The buffer containing elements to draw 
	 */
	void DrawWindows_Private(FSlateDrawBuffer& InWindowDrawBuffer);

	/**
	 * Delete the updateable textures we've marked for delete that have already had their GPU resources released, but may
	 * have already been used on the game thread at the time they were released.
	 */
	void CleanUpdatableTextures();

	void OnVirtualDesktopSizeChanged(const FDisplayMetrics& NewDisplayMetric);

	/** A mapping of SWindows to their RHI implementation */
	TMap<const SWindow*, FSlateViewportInfo*> WindowToViewportInfo;

	/** Keep a pointer around for when we have deferred drawing happening */
	FSlateDrawBuffer* EnqueuedWindowDrawBuffer = nullptr;

	/** Double buffered draw buffers so that the rendering thread can be rendering windows while the game thread is setting up for next frame */
	FSlateDrawBuffer DrawBuffers[NumDrawBuffers];

	/** The draw buffer which is currently free for use by the game thread */
	uint8 FreeBufferIndex = 0;

	/** Element batcher which renders draw elements */
	TUniquePtr<FSlateElementBatcher> ElementBatcher;

	/** Texture manager for accessing textures on the game thread */
	TSharedPtr<FSlateRHIResourceManager> ResourceManager;

	/** Drawing policy */
	TSharedPtr<FSlateRHIRenderingPolicy> RenderingPolicy;

	TArray<TSharedPtr<FSlateDynamicImageBrush>> DynamicBrushesToRemove[NumDrawBuffers];

	struct
	{
		TArray<FSlateCachedFastPathRenderingData*, FConcurrentLinearArrayAllocator> CachedFastPathRenderingData;
		TArray<FSlateCachedElementData*, FConcurrentLinearArrayAllocator> CachedElementData;

		bool IsEmpty() const { return CachedFastPathRenderingData.IsEmpty() && CachedElementData.IsEmpty(); }

	} PendingDeletes;

	void FlushPendingDeletes();

	TArray<FRenderThreadUpdateContext, FConcurrentLinearArrayAllocator> DeferredUpdateContexts;

	bool bIsStandaloneStereoOnlyDevice = false;
	bool bUpdateHDRDisplayInformation = false;
	ESlatePostRT bShrinkPostBufferRequested = ESlatePostRT::None;
	uint64 LastFramesPostBufferUsed[(uint8)ESlatePostRT::Num];
	FRenderCommandFence SlatePostRTFences[(uint8)ESlatePostRT::Num];

	struct
	{
		TStaticArray<uint64, (int32)ESlatePostRT::Num> LastUsedFrameCounter{ InPlace, 0 };
	
	} PostProcessRenderTargets;

	struct
	{
		FIntRect ViewRect;
		FSlateViewportInfo* ViewportToCapture = nullptr;
		TArray<FColor>* ColorData = nullptr;
		TArray<FLinearColor>* ColorDataHDR = nullptr;

	} ScreenshotState;

	void OnSceneRemoved(FSceneInterface* Scene);

	/** These are state management variables for Scenes on the game thread. A similar copy exists on the RHI Rendering Policy for the rendering thread.*/
	TArray<FSceneInterface*> ActiveScenes;
	int32 CurrentSceneIndex = -1;
	FDelegateHandle SceneRemovedHandle;
	
	/** Version that increments when it is okay to clean up older cached resources */
	uint32 ResourceVersion = 0;

	friend FSlateViewportInfo;
};
