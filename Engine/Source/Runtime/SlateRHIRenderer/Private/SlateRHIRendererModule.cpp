// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "Fonts/FontTypes.h"
#include "Fonts/FontCache.h"
#include "Rendering/RenderingCommon.h"
#include "Rendering/DrawElements.h"
#include "Rendering/SlateRenderer.h"
#include "Interfaces/ISlate3DRenderer.h"
#include "Interfaces/ISlateRHIRendererModule.h"
#include "SlateRHIFontTexture.h"
#include "SlateRHIResourceManager.h"
#include "SlateRHIRenderer.h"
#include "Slate3DRenderer.h"
#include "SlateUpdatableBuffer.h"

class FSlateRHIFontAtlasFactory : public ISlateFontAtlasFactory
{
public:
	FSlateRHIFontAtlasFactory()
	{
		auto GetAtlasSizeFromConfig = [](const TCHAR* InConfigKey, const FString& InConfigFilename, int32& OutAtlasSize)
		{
			if (GConfig)
			{
				GConfig->GetInt(TEXT("SlateRenderer"), InConfigKey, OutAtlasSize, InConfigFilename);
				if (GConfig->GetInt(TEXT("SlateRenderer"), TEXT("FontAtlasSize"), OutAtlasSize, InConfigFilename))
				{
					UE_LOGF(LogCore, Warning, "The 'FontAtlasSize' setting for 'SlateRenderer' is deprecated. Use '%ls' instead.", InConfigKey);
				}
			}
			OutAtlasSize = FMath::Clamp(OutAtlasSize, 128, 2048);
		};

		GrayscaleAtlasSize = GIsEditor ? 2048 : 1024;
		ColorAtlasSize = 512;
		SdfAtlasSize = GIsEditor ? 1024 : 512;
		{
			const FString& ConfigFilename = GIsEditor ? GEditorIni : GEngineIni;
			GetAtlasSizeFromConfig(TEXT("GrayscaleFontAtlasSize"), ConfigFilename, GrayscaleAtlasSize);
			GetAtlasSizeFromConfig(TEXT("ColorFontAtlasSize"), ConfigFilename, ColorAtlasSize);
			GetAtlasSizeFromConfig(TEXT("SdfFontAtlasSize"), ConfigFilename, SdfAtlasSize);
		}
	}

	virtual ~FSlateRHIFontAtlasFactory()
	{
	}

	virtual FIntPoint GetAtlasSize(ESlateFontAtlasContentType InContentType) const override
	{
		switch (InContentType)
		{
			case ESlateFontAtlasContentType::Alpha:
				return FIntPoint(GrayscaleAtlasSize, GrayscaleAtlasSize);
			case ESlateFontAtlasContentType::Color:
				return FIntPoint(ColorAtlasSize, ColorAtlasSize);
			case ESlateFontAtlasContentType::Msdf:
				return FIntPoint(SdfAtlasSize, SdfAtlasSize);
			default:
				checkNoEntry();
				// Default to Color
				return FIntPoint(ColorAtlasSize, ColorAtlasSize);
		}
	}

	virtual TSharedRef<FSlateFontAtlas> CreateFontAtlas(ESlateFontAtlasContentType InContentType) const override
	{
		const FIntPoint AtlasSize = GetAtlasSize(InContentType);
		return MakeShareable(new FSlateFontAtlasRHI(AtlasSize.X, AtlasSize.Y, InContentType, ESlateTextureAtlasPaddingStyle::PadWithZero));
	}

	virtual TSharedPtr<ISlateFontTexture> CreateNonAtlasedTexture(const uint32 InWidth, const uint32 InHeight, ESlateFontAtlasContentType InContentType, const TArray<uint8>& InRawData) const override
	{
		if (GIsEditor)
		{
			const FIntPoint AtlasSize = GetAtlasSize(InContentType);
			const uint32 MaxFontTextureDimension = FMath::Min(AtlasSize.Y * 4u, GetMax2DTextureDimension()); // Don't allow textures greater than 4x our atlas size, but still honor the platform limit
			if (InWidth <= MaxFontTextureDimension && InHeight <= MaxFontTextureDimension)
			{
				return MakeShareable(new FSlateFontTextureRHI(InWidth, InHeight, InContentType, InRawData));
			}
		}
		return nullptr;
	}

private:
	/** Size of each font texture, width and height */
	int32 GrayscaleAtlasSize;
	int32 ColorAtlasSize;
	int32 SdfAtlasSize;
};


/**
 * Implements the Slate RHI Renderer module.
 */
class FSlateRHIRendererModule
	: public ISlateRHIRendererModule
{
public:

	// ISlateRHIRendererModule interface
	virtual TSharedRef<FSlateRenderer> CreateSlateRHIRenderer( ) override
	{
		ConditionalCreateResources();

		return MakeShareable( new FSlateRHIRenderer( SlateFontServices.ToSharedRef(), ResourceManager.ToSharedRef() ) );
	}

	virtual TSharedRef<ISlate3DRenderer, ESPMode::ThreadSafe> CreateSlate3DRenderer(bool bUseGammaCorrection) override
	{
		ConditionalCreateResources();

		return MakeShareable(new FSlate3DRenderer(SlateFontServices.ToSharedRef(), ResourceManager.ToSharedRef(), bUseGammaCorrection), [=] (FSlate3DRenderer* Renderer) {
			Renderer->Cleanup();
		});
	}

	virtual TSharedRef<ISlateFontAtlasFactory> CreateSlateFontAtlasFactory() override
	{
		return MakeShareable(new FSlateRHIFontAtlasFactory);
	}

	virtual TSharedRef<ISlateUpdatableInstanceBuffer> CreateInstanceBuffer( int32 InitialInstanceCount ) override
	{
		return MakeShareable( new FSlateUpdatableInstanceBuffer(InitialInstanceCount) );
	}

	virtual void StartupModule( ) override { }
	virtual void ShutdownModule( ) override { }

private:
	/** Creates resource managers if they do not exist */
	void ConditionalCreateResources()
	{
		if( !ResourceManager.IsValid() )
		{
			ResourceManager = MakeShareable( new FSlateRHIResourceManager );
		}

		if( !SlateFontServices.IsValid() )
		{
			const TSharedRef<FSlateFontCache> GameThreadFontCache = MakeShareable(new FSlateFontCache(MakeShareable(new FSlateRHIFontAtlasFactory), ESlateTextureAtlasThreadId::Game));
			const TSharedRef<FSlateFontCache> RenderThreadFontCache = MakeShareable(new FSlateFontCache(MakeShareable(new FSlateRHIFontAtlasFactory), ESlateTextureAtlasThreadId::Render));

			SlateFontServices = MakeShareable(new FSlateFontServices(GameThreadFontCache, RenderThreadFontCache));
		}
	}
private:
	/** Resource manager used for all renderers */
	TSharedPtr<FSlateRHIResourceManager> ResourceManager;

	/** Font services used for all renderers */
	TSharedPtr<FSlateFontServices> SlateFontServices;
};


IMPLEMENT_MODULE( FSlateRHIRendererModule, SlateRHIRenderer ) 
