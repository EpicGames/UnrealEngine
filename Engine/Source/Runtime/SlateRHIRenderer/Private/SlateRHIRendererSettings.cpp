// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateRHIRendererSettings.h"
#include "Engine/TextureRenderTarget2D.h"
#include "FX/SlateRHIPostBufferProcessor.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateRHIRendererSettings)

static TAutoConsoleVariable<int32> CVarDefaultEnablePostRenderTarget_0(
	TEXT("Slate.DefaultEnablePostRenderTarget_0"),
	1,
	TEXT("Experimental. Set true to enable slate post render target 0"),
	ECVF_ReadOnly);

FSlatePostSettings::FSlatePostSettings()
	: bEnabled(false)
	, Resolution(ESlatePostResolution::Full)
	, PostProcessorClass(nullptr)
	, PathToSlatePostRT(FString())
	, CachedSlatePostRT(nullptr)
	, bLoadAttempted(false)
{
}

USlateRHIRendererSettings::USlateRHIRendererSettings()
{
	SlatePostSettings.Add(ESlatePostRT::ESlatePostRT_0, FSlatePostSettings());
	SlatePostSettings.Add(ESlatePostRT::ESlatePostRT_1, FSlatePostSettings());
	SlatePostSettings.Add(ESlatePostRT::ESlatePostRT_2, FSlatePostSettings());
	SlatePostSettings.Add(ESlatePostRT::ESlatePostRT_3, FSlatePostSettings());
	SlatePostSettings.Add(ESlatePostRT::ESlatePostRT_4, FSlatePostSettings());

	// By default, enable the first post RT
	SlatePostSettings[ESlatePostRT::ESlatePostRT_0].bEnabled = CVarDefaultEnablePostRenderTarget_0.GetValueOnAnyThread();

	// Hardcoded paths to engine assets
	SlatePostSettings[ESlatePostRT::ESlatePostRT_0].PathToSlatePostRT = "/Engine/EngineResources/SlatePost0_RT.SlatePost0_RT";
	SlatePostSettings[ESlatePostRT::ESlatePostRT_1].PathToSlatePostRT = "/Engine/EngineResources/SlatePost1_RT.SlatePost1_RT";
	SlatePostSettings[ESlatePostRT::ESlatePostRT_2].PathToSlatePostRT = "/Engine/EngineResources/SlatePost2_RT.SlatePost2_RT";
	SlatePostSettings[ESlatePostRT::ESlatePostRT_3].PathToSlatePostRT = "/Engine/EngineResources/SlatePost3_RT.SlatePost3_RT";
	SlatePostSettings[ESlatePostRT::ESlatePostRT_4].PathToSlatePostRT = "/Engine/EngineResources/SlatePost4_RT.SlatePost4_RT";
}

USlateRHIRendererSettings::~USlateRHIRendererSettings()
{
	// No need to remove from root during the exit purge as all objects will destroyed regardless of their flags 
	// and it's not safe to access any other UObjects from UObject destructors
	if (!GExitPurge)
	{
		for (TPair<ESlatePostRT, FSlatePostSettings>& SlatePostSetting : SlatePostSettings)
		{
			FSlatePostSettings& PostSetting = SlatePostSetting.Value;

			UObject* SlatePostBuffer = PostSetting.CachedSlatePostRT;
			if (SlatePostBuffer)
			{
				SlatePostBuffer->RemoveFromRoot();
			}
		}
	}
}

void USlateRHIRendererSettings::BeginDestroy()
{
	// Flush rendering commands since these settings can be used in render thread
	FlushRenderingCommands();

	Super::BeginDestroy();
}

FSlatePostSettings& USlateRHIRendererSettings::GetMutableSlatePostSetting(ESlatePostRT InPostBufferBit)
{
	return SlatePostSettings[InPostBufferBit];
}

const FSlatePostSettings& USlateRHIRendererSettings::GetSlatePostSetting(ESlatePostRT InPostBufferBit) const
{
	return SlatePostSettings[InPostBufferBit];
}

UTextureRenderTarget2D* USlateRHIRendererSettings::TryGetPostBufferRT(ESlatePostRT InPostBufferBit) const
{
	return SlatePostSettings[InPostBufferBit].CachedSlatePostRT;
}

UTextureRenderTarget2D* USlateRHIRendererSettings::LoadGetPostBufferRT(ESlatePostRT InPostBufferBit)
{
	FSlatePostSettings& SlatePostSetting = SlatePostSettings[InPostBufferBit];

	UTextureRenderTarget2D* Result = nullptr;

	if (SlatePostSetting.bEnabled)
	{
		Result = SlatePostSetting.CachedSlatePostRT;

		if (!Result && !SlatePostSetting.bLoadAttempted)
		{
			Result = LoadObject<UTextureRenderTarget2D>(nullptr, *SlatePostSetting.PathToSlatePostRT, nullptr, LOAD_None, nullptr);

			if (Result)
			{
				Result->AddToRoot();
				SlatePostSetting.CachedSlatePostRT = Result;
			}

			SlatePostSetting.bLoadAttempted = true;
		}
	}

	return Result;
}

const TMap<ESlatePostRT, FSlatePostSettings>& USlateRHIRendererSettings::GetSlatePostSettings() const
{
	return SlatePostSettings;
}

int32 USlateRHIRendererSettings::GetSlatePostBufferDownscaleFactor(ESlatePostRT InPostBufferBit) const
{
	switch (GetSlatePostSetting(InPostBufferBit).Resolution)
	{
		case ESlatePostResolution::Full:
			return 1;
		case ESlatePostResolution::Half:
			return 2;
	}
	checkNoEntry(); // Missing case label?
	return 1;
}
