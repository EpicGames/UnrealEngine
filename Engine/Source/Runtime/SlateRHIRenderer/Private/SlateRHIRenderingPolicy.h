// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "GameTime.h"
#include "RendererInterface.h"
#include "Rendering/RenderingCommon.h"
#include "Rendering/SlateRendererTypes.h"
#include "Rendering/ShaderResourceManager.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingPolicy.h"
#include "Layout/Clipping.h"
#include "SlateElementVertexBuffer.h"
#include "SlateRHIResourceManager.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "Engine/TextureLODSettings.h"

class FSlateFontServices;
class FSlateRHIResourceManager;
class FSlatePostProcessor;
class UDeviceProfile;
class FSlateElementPS;
class FSlateMaterialShaderPS;
class FSlateMaterialShaderVS;
struct FMaterialShaderTypes;
struct IPooledRenderTarget;

class FSlateRHIRenderingPolicy final : public FSlateRenderingPolicy
{
public:
	FSlateRHIRenderingPolicy(TSharedRef<FSlateFontServices> InSlateFontServices, TSharedRef<FSlateRHIResourceManager> InResourceManager);

	virtual TSharedRef<FSlateShaderResourceManager> GetResourceManager() const override { return ResourceManager; }
	virtual bool IsVertexColorInLinearSpace() const override { return false; }

	FSlateRHIResourceManager& GetResourceManagerRHI() const { return ResourceManager.Get(); }

	virtual void AddSceneAt(FSceneInterface* Scene, int32 Index) override;
	virtual void ClearScenes() override;

private:
	TSharedRef<FSlateRHIResourceManager> ResourceManager;
};

struct FSlateElementsBuffers
{
	FRDGBuffer* VertexBuffer = nullptr;
	FRDGBuffer* IndexBuffer = nullptr;
};

FSlateElementsBuffers BuildSlateElementsBuffers(FRDGBuilder& GraphBuilder, FSlateBatchData& BatchData);

struct FSlateDrawElementsPassInputs
{
	FRDGTexture* StencilTexture = nullptr;
	FRDGTexture* ElementsTexture = nullptr;
	FRDGTexture* SceneViewportTexture = nullptr;
	ERenderTargetLoadAction ElementsLoadAction = ERenderTargetLoadAction::ENoAction;
	FSlateElementsBuffers ElementsBuffers;
	FMatrix44f ElementsMatrix;
	FVector2f ElementsOffset = FVector2f::Zero();
	FIntRect SceneViewRect;
	FIntPoint CursorPosition = FIntPoint::ZeroValue;
	FGameTime Time;
	EDisplayColorGamut HDRDisplayColorGamut = EDisplayColorGamut::sRGB_D65;
	float HDRPaperWhiteInNits = 203.f;
	ESlatePostRT UsedSlatePostBuffers = ESlatePostRT::None;
	float ViewportScaleUI = 1.0f;
	bool bWireframe = false;
	bool bElementsTextureIsHDRDisplay = false;
	bool bAllowGammaCorrection = true;
	bool bAllowColorDeficiencyCorrection = true;
};

void AddSlateDrawElementsPass(
	FRDGBuilder& GraphBuilder,
	const FSlateRHIRenderingPolicy& RenderingPolicy,
	const FSlateDrawElementsPassInputs& Inputs,
	TConstArrayView<FSlateRenderBatch> RenderBatches,
	int32 FirstBatchIndex,
    TInterval<int32> LayerRange = TInterval<int32>(std::numeric_limits<int32>::min(), std::numeric_limits<int32>::max()));
