// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/SlateRHIRenderingPolicyInterface.h"