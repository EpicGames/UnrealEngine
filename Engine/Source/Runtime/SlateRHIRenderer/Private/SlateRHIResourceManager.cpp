// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateRHIResourceManager.h"
#include "RenderingThread.h"
#include "Engine/Texture.h"
#include "Misc/FileHelper.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Async/ParallelFor.h"
#include "Modules/ModuleManager.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/ISlateStyle.h"
#include "Rendering/SlateRenderer.h"
#include "EngineGlobals.h"
#include "Engine/Texture2D.h"
#include "RenderUtils.h"
#include "Engine/Engine.h"
#include "Slate/SlateTextures.h"
#include "SlateRHITextureAtlas.h"
#include "SlateNativeTextureResource.h"
#include "SlateUTextureResource.h"
#include "SlateMaterialResource.h"
#include "Slate/SlateTextureAtlasInterface.h"
#include "SlateAtlasedTextureResource.h"
#include "Types/SlateVector2.h"
#include "ImageUtils.h"
#include "Async/ParallelFor.h"
#include "Rendering/SlateVectorGraphicsCache.h"

#define LOCTEXT_NAMESPACE "Slate"

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Num Texture Atlases"), STAT_SlateNumTextureAtlases, STATGROUP_SlateMemory);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Num Non-Atlased Textures"), STAT_SlateNumNonAtlasedTextures, STATGROUP_SlateMemory);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Num Dynamic Textures"), STAT_SlateNumDynamicTextures, STATGROUP_SlateMemory);
DECLARE_CYCLE_STAT(TEXT("GetResource Time"), STAT_SlateGetResourceTime, STATGROUP_Slate);

static TAutoConsoleVariable<bool> CVarSlateRHIResourceManagerLockWhenGCing(
	TEXT("Slate.ResourceManager.LockResourceDuringGC"),
	true,
	TEXT("Lock the Slate RHI Resource Manager when GCing and when the loading screen has ownership to prevent multithreaded access to the resources."));

class FSlateRHITextureAtlasFactory : public ISlateTextureAtlasFactory
{
public:
	virtual TUniquePtr<FSlateTextureAtlas> CreateTextureAtlas(int32 AtlasSize, int32 AtlasStride, ESlateTextureAtlasPaddingStyle PaddingStyle, bool bUpdatesAfterInitialization) const
	{
		return CreateTextureAtlasInternal(AtlasSize, AtlasStride, PaddingStyle, bUpdatesAfterInitialization);
	}

	virtual TUniquePtr<FSlateShaderResource> CreateNonAtlasedTexture(const uint32 InWidth, const uint32 InHeight, const TArray<uint8>& InRawData) const
	{
		uint32 MaxSize = FMath::Max(InWidth, InHeight);

		const uint32 MaxTextureDimension = FMath::Min(MaxSize, GetMax2DTextureDimension());
		if (InWidth <= MaxTextureDimension && InHeight <= MaxTextureDimension)
		{
			TSharedPtr<FSlateTextureData, ESPMode::ThreadSafe> TextureData = MakeShared<FSlateTextureData, ESPMode::ThreadSafe>(InWidth, InHeight, GPixelFormats[PF_B8G8R8A8].BlockBytes, InRawData);

			TUniquePtr<FSlateTexture2DRHIRef> NewTexture = MakeUnique<FSlateTexture2DRHIRef>(InWidth, InHeight, PF_B8G8R8A8, TextureData, TexCreate_SRGB | TexCreate_ShaderResource);
			BeginInitResource(NewTexture.Get());

			return NewTexture;
		}

		return nullptr;
	}

	virtual void ReleaseTextureAtlases(const TArray<TUniquePtr<FSlateTextureAtlas>>& InTextureAtlases, const TArray<TUniquePtr<FSlateShaderResource>>& InNonAtlasedTextures, const bool bWaitForRelease) const
	{
		for (const TUniquePtr<FSlateTextureAtlas>& Atlas : InTextureAtlases)
		{
			Atlas->ReleaseResources();
		}

		for (const TUniquePtr<FSlateShaderResource>& Texture : InNonAtlasedTextures)
		{
			FSlateTexture2DRHIRef* RHITexture = static_cast<FSlateTexture2DRHIRef*>(Texture.Get());
			BeginReleaseResource(RHITexture);
		}

		if (bWaitForRelease)
		{
			FRenderCommandFence Fence;
			Fence.BeginFence();
			Fence.Wait();
		}
	}

	static TUniquePtr<FSlateTextureAtlasRHI> CreateTextureAtlasInternal(int32 AtlasSize, int32 AtlasStride, ESlateTextureAtlasPaddingStyle PaddingStyle, bool bUpdatesAfterInitialization)
	{
		return MakeUnique<FSlateTextureAtlasRHI>(AtlasSize, AtlasSize, PaddingStyle, bUpdatesAfterInitialization);
	}
};

FDynamicResourceMap::FDynamicResourceMap()
{
}

TSharedPtr<FSlateDynamicTextureResource> FDynamicResourceMap::GetDynamicTextureResource( FName ResourceName ) const
{
	return NativeTextureMap.FindRef( ResourceName );
}

TSharedPtr<FSlateUTextureResource> FDynamicResourceMap::GetUTextureResource( UTexture* TextureObject ) const
{
	if(TextureObject)
	{
		return TextureMap.FindRef(TextureObject);
	}

	return nullptr;
}

TSharedPtr<FSlateAtlasedTextureResource> FDynamicResourceMap::GetAtlasedTextureResource(UTexture* InObject) const
{
	if ( InObject )
	{
		return ObjectMap.FindRef(InObject);
	}

	return nullptr;
}

TSharedPtr<FSlateMaterialResource> FDynamicResourceMap::GetMaterialResource( const FMaterialKey& InKey ) const
{
	return MaterialMap.FindRef( InKey );
}


void FDynamicResourceMap::AddDynamicTextureResource( FName ResourceName, TSharedRef<FSlateDynamicTextureResource> InResource )
{
	NativeTextureMap.Add( ResourceName, InResource );
}

void FDynamicResourceMap::AddUTextureResource( UTexture* TextureObject, TSharedRef<FSlateUTextureResource> InResource)
{
	if ( TextureObject )
	{
		check(TextureObject == InResource->GetTextureObject());
		TextureMap.Add(TextureObject, InResource);
	}
}

void FDynamicResourceMap::AddMaterialResource( const FMaterialKey& InKey, TSharedRef<FSlateMaterialResource> InMaterialResource )
{
	check(InKey.Material == InMaterialResource->GetMaterialObject() );
	MaterialMap.Add(InKey, InMaterialResource);
}

void FDynamicResourceMap::RemoveDynamicTextureResource(FName ResourceName)
{
	NativeTextureMap.Remove(ResourceName);
}

void FDynamicResourceMap::RemoveUTextureResource(UTexture* TextureObject)
{
	if (TextureObject)
	{
		TextureMap.Remove(TextureObject);
	}
}

void FDynamicResourceMap::RemoveMaterialResource( const FMaterialKey& InKey )
{
	MaterialMap.Remove(InKey);
}

void FDynamicResourceMap::AddAtlasedTextureResource(UTexture* TextureObject, TSharedRef<FSlateAtlasedTextureResource> InResource)
{
	if ( TextureObject )
	{
		ObjectMap.Add(TextureObject, InResource);
	}
}

void FDynamicResourceMap::RemoveAtlasedTextureResource(UTexture* TextureObject)
{
	ObjectMap.Remove(TextureObject);
}

void FDynamicResourceMap::Empty()
{
	EmptyUTextureResources();
	EmptyMaterialResources();
	EmptyDynamicTextureResources();
}

void FDynamicResourceMap::EmptyDynamicTextureResources()
{
	NativeTextureMap.Empty();
}

void FDynamicResourceMap::EmptyUTextureResources()
{
	TextureMap.Empty();
}

void FDynamicResourceMap::EmptyMaterialResources()
{
	MaterialMap.Empty();
}

void FDynamicResourceMap::ReleaseResources()
{
	for (TMap<FName, TSharedPtr<FSlateDynamicTextureResource> >::TIterator It(NativeTextureMap); It; ++It)
	{
		BeginReleaseResource(It.Value()->RHIRefTexture);
	}
	
	for ( FTextureResourceMap::TIterator It(TextureMap); It; ++It )
	{
		It.Value()->ResetTexture();
	}
}

void FDynamicResourceMap::RemoveExpiredTextureResources(TArray< TSharedPtr<FSlateUTextureResource> >& RemovedTextures)
{
	for (FTextureResourceMap::TIterator It(TextureMap); It; ++It)
	{
		TWeakObjectPtr<UTexture>& Key = It.Key();
		if (!Key.IsValid())
		{
			RemovedTextures.Push(It.Value());
			It.Value()->ResetTexture();
			It.RemoveCurrent();
		}
	}
}

void FDynamicResourceMap::RemoveExpiredMaterialResources(TArray< TSharedPtr<FSlateMaterialResource> >& RemovedMaterials)
{
	for (FMaterialResourceMap::TIterator It(MaterialMap); It; ++It)
	{
		FMaterialKey& Key = It.Key();
		if (Key.Material.ResolveObjectPtr() == nullptr)
		{
			RemovedMaterials.Push(It.Value());
			It.Value()->ResetMaterial();
			It.RemoveCurrent();
		}
	}
}

FSlateRHIResourceManager::FSlateRHIResourceManager()
	: bResourceCriticalSectionLockedForGC(false)
	, bExpiredResourcesNeedCleanup(false)
	, BadResourceTexture(nullptr)
	, DeleteResourcesCommand(
		TEXT("Slate.DeleteResources"),
		*LOCTEXT("CommandText_DeleteResources", "Flushes and deletes all resources created by Slate's RHI Resource Manager.").ToString(),
		FConsoleCommandDelegate::CreateRaw(this, &FSlateRHIResourceManager::DeleteBrushResourcesCommand))
{
	FCoreDelegates::OnPreExit.AddRaw(this, &FSlateRHIResourceManager::OnAppExit);
	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddRaw(this, &FSlateRHIResourceManager::OnPreGarbageCollect);
	FCoreUObjectDelegates::GetPostGarbageCollect().AddRaw(this, &FSlateRHIResourceManager::OnPostGarbageCollect);

	MaxAltasedTextureSize = FIntPoint(256, 256);
	if (GIsEditor)
	{
		AtlasSize = 2048;
	}
	else
	{
		AtlasSize = 1024;
		if (GConfig)
		{
			int32 RequestedSize = 1024;
			GConfig->GetInt(TEXT("SlateRenderer"), TEXT("TextureAtlasSize"), RequestedSize, GEngineIni);
			AtlasSize = FMath::Clamp<uint32>(RequestedSize, 0, 2048);

			int32 MaxAtlasedTextureWidth = 256;
			int32 MaxAtlasedTextureHeight = 256;
			GConfig->GetInt(TEXT("SlateRenderer"), TEXT("MaxAtlasedTextureWidth"), MaxAtlasedTextureWidth, GEngineIni);
			GConfig->GetInt(TEXT("SlateRenderer"), TEXT("MaxAtlasedTextureHeight"), MaxAtlasedTextureHeight, GEngineIni);

			// Max texture size cannot be larger than the max size of the atlas
			MaxAltasedTextureSize.X = FMath::Clamp<int32>(MaxAtlasedTextureWidth, 0, AtlasSize);
			MaxAltasedTextureSize.Y = FMath::Clamp<int32>(MaxAtlasedTextureHeight, 0, AtlasSize);
		}
	}

	CreateVectorGraphicsCache();
}

FSlateRHIResourceManager::~FSlateRHIResourceManager()
{
	FCoreDelegates::OnPreExit.RemoveAll( this );
	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().RemoveAll(this);
	FCoreUObjectDelegates::GetPostGarbageCollect().RemoveAll(this);

	if (GIsRHIInitialized)
	{
		FlushRenderingCommands();

		DeleteResources();
	}
}

void FSlateRHIResourceManager::CreateVectorGraphicsCache()
{
	VectorGraphicsCache = MakeUnique<FSlateVectorGraphicsCache>(MakeShared<FSlateRHITextureAtlasFactory>());
}

void FSlateRHIResourceManager::OnPreGarbageCollect()
{
	check(bResourceCriticalSectionLockedForGC == false);
	bResourceCriticalSectionLockedForGC = (GSlateLoadingThreadId != 0) && CVarSlateRHIResourceManagerLockWhenGCing.GetValueOnGameThread();
	if (bResourceCriticalSectionLockedForGC)
	{
		ResourceCriticalSection.Lock();
	}
}

void FSlateRHIResourceManager::OnPostGarbageCollect()
{
	if (bResourceCriticalSectionLockedForGC)
	{
		CleanupExpiredResources();
		ResourceCriticalSection.Unlock();
		bResourceCriticalSectionLockedForGC = false;
	}
	else
	{
		TryToCleanupExpiredResources(true);
	}
}

void FSlateRHIResourceManager::TryToCleanupExpiredResources(bool bForceCleanup)
{
	if (!(IsInGameThread() && !IsInSlateThread()))
	{
		return;
	}

	if (bForceCleanup || bExpiredResourcesNeedCleanup)
	{
		if (ResourceCriticalSection.TryLock())
		{
			CleanupExpiredResources();
			ResourceCriticalSection.Unlock();
		}
		else
		{
			// It's possible that during a slate loading thread we might both need to load something, be doing garbage collection,
			// and be midway rendering the loading screen.  Composite font loads being an example, being loaded for subtitles on
			// a movie.
			//
			// When this happens - to avoid a potential deadlock, we just queue up attempting to cleanup expired resources until the
			// next time we tick the resource manager when we fail to acquire the lock on the resource manager.
			bExpiredResourcesNeedCleanup = true;
		}
	}
}

void FSlateRHIResourceManager::CleanupExpiredResources()
{
	bExpiredResourcesNeedCleanup = false;

	DynamicResourceMap.RemoveExpiredTextureResources(UTextureFreeList);
	DynamicResourceMap.RemoveExpiredMaterialResources(MaterialResourceFreeList);
}

int32 FSlateRHIResourceManager::GetNumAtlasPages() const
{
	return PrecachedTextureAtlases.Num() + VectorGraphicsCache->GetNumAtlasPages();
}

FSlateShaderResource* FSlateRHIResourceManager::GetAtlasPageResource(const int32 InIndex) const
{
	return InIndex < PrecachedTextureAtlases.Num() ? PrecachedTextureAtlases[InIndex]->GetAtlasTexture() : VectorGraphicsCache->GetAtlasPageResource(InIndex - PrecachedTextureAtlases.Num());
}

bool FSlateRHIResourceManager::IsAtlasPageResourceAlphaOnly(const int32 InIndex) const
{
	return false;
}

#if WITH_ATLAS_DEBUGGING
FAtlasSlotInfo FSlateRHIResourceManager::GetAtlasSlotInfoAtPosition(FIntPoint InPosition, int32 AtlasIndex) const
{
	const FSlateTextureAtlas* Atlas = nullptr;

	bool bIsPrecachedTextureAtlases = PrecachedTextureAtlases.IsValidIndex(AtlasIndex);
	if (bIsPrecachedTextureAtlases)
	{
		Atlas = PrecachedTextureAtlases[AtlasIndex];
	}
	else
	{
		Atlas = VectorGraphicsCache->GetAtlas(AtlasIndex - PrecachedTextureAtlases.Num());
	}

	FAtlasSlotInfo NewInfo;
	if (Atlas)
	{
		const FAtlasedTextureSlot* Slot = Atlas->GetSlotAtPosition(InPosition);
		if (Slot)
		{
			NewInfo.AtlasSlotRect = FSlateRect(FVector2f((float)Slot->X, (float)Slot->Y), FVector2f((float)(Slot->X + Slot->Width), (float)(Slot->Y + Slot->Height)));
			NewInfo.TextureName = bIsPrecachedTextureAtlases ? AtlasDebugData.FindRef(Slot) : VectorGraphicsCache->GetAtlasDebugData(Slot);
		}
	}

	return NewInfo;
}
#endif

void FSlateRHIResourceManager::Tick(float DeltaSeconds)
{
	TryToCleanupExpiredResources(false);
}

void FSlateRHIResourceManager::CreateTextures( const TArray< const FSlateBrush* >& Resources )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSlateRHIResourceManager::CreateTextures)

	struct FLoadTextureInfo : public FNewTextureInfo
	{
		bool bSucceeded = false;
		const FSlateBrush* Brush;
	};

	TMap<FName, FLoadTextureInfo> TextureInfoMap;
	for( int32 ResourceIndex = 0; ResourceIndex < Resources.Num(); ++ResourceIndex )
	{
		const FSlateBrush& Brush = *Resources[ResourceIndex];
		const FName TextureName = Brush.GetResourceName();
		if(Brush.GetImageType() != ESlateBrushImageType::Vector && TextureName != NAME_None && !Brush.HasUObject() && !Brush.IsDynamicallyLoaded() && !ResourceMap.Contains(TextureName) )
		{
			// Find the texture or add it if it doesn't exist (only load the texture once)
			FLoadTextureInfo& Info = TextureInfoMap.FindOrAdd( TextureName );

			Info.Brush = &Brush;
			Info.bSrgb = (Brush.ImageType != ESlateBrushImageType::Linear);

			// Only atlas the texture if none of the brushes that use it tile it and the image is srgb
		
			Info.bShouldAtlas &= ( Brush.Tiling == ESlateBrushTileType::NoTile && Info.bSrgb && AtlasSize > 0 );
		}
	}

	// ImageWrapper module must be loaded on game thread before going multi-threaded
	//	-> note this is the first place that ImageWrapper module is loaded in LaunchEngineLoop
	FModuleManager::Get().LoadModuleChecked(FName("ImageWrapper"));

	TArray<FName> TexturesToLoad;
	TextureInfoMap.GenerateKeyArray(TexturesToLoad);

	const uint32 BytesPerPixel = GPixelFormats[PF_R8G8B8A8].BlockBytes;
	check( BytesPerPixel == 4 );

	ParallelFor(
		TexturesToLoad.Num(),
		[&](int32 TextureIndex)
		{
			// TextureInfoMap is being accessed multi-threaded, it must not be changed during this phase
			FName& TextureName = TexturesToLoad[TextureIndex];
			FLoadTextureInfo& Info = TextureInfoMap.FindChecked(TextureName);

			uint32 Width = 0;
			uint32 Height = 0;
			TArray<uint8> RawData;
			Info.bSucceeded = LoadTexture(*Info.Brush, Width, Height, RawData);

			if ( Info.bSucceeded )
			{
				Info.TextureData = MakeShareable(new FSlateTextureData(Width, Height, BytesPerPixel, MoveTemp(RawData)));

				const bool bTooLargeForAtlas = (Width >= (uint32)MaxAltasedTextureSize.X || Height >= (uint32)MaxAltasedTextureSize.Y || Width >= AtlasSize || Height >= AtlasSize);
				Info.bShouldAtlas &= !bTooLargeForAtlas;
			}
		},
		EParallelForFlags::Unbalanced
	);

	// Remove any texture that didn't succeed
	for (FName& TextureName : TexturesToLoad)
	{
		FLoadTextureInfo& Info = TextureInfoMap.FindChecked(TextureName);
		if (!Info.bSucceeded || !ensureMsgf(Info.TextureData->GetRawBytes().Num() > 0, TEXT("Slate resource: (%s) contains no data"), *TextureName.ToString()))
		{
			TextureInfoMap.Remove(TextureName);
		}
	}

	// Sort textures by size.  The largest textures are atlased first which creates a more compact atlas
	TextureInfoMap.ValueSort( FCompareFNewTextureInfoByTextureSize() );

	for( TMap<FName,FLoadTextureInfo>::TConstIterator It(TextureInfoMap); It; ++It )
	{
		const FLoadTextureInfo& Info = It.Value();
		FName TextureName = It.Key();

		checkSlow( TextureName != NAME_None );

		FSlateShaderResourceProxy* NewTexture = GenerateTextureResource( Info, TextureName );

		ResourceMap.Add( TextureName, NewTexture );
	}
}

bool FSlateRHIResourceManager::LoadTexture( const FSlateBrush& InBrush, uint32& Width, uint32& Height, TArray<uint8>& DecodedImage )
{
	FString ResourcePath = GetResourcePath( InBrush );

	return LoadTexture(InBrush.GetResourceName(), ResourcePath, Width, Height, DecodedImage);
}

/** 
 * Loads a UTexture2D from a package and stores it in the cache
 *
 * @param TextureName	The name of the texture to load
 */
bool FSlateRHIResourceManager::LoadTexture( const FName& TextureName, const FString& ResourcePath, 
	uint32& OutWidth, uint32& OutHeight, TArray<uint8>& OutDecodedImage )
{
	OutWidth = OutHeight = 0;

	FImage Image;
	if ( ! FImageUtils::LoadImage(*ResourcePath,Image) )
	{
		UE_CLOGF(!IsRunningCommandlet(), LogSlate, Log, "Could not load file for Slate resource: [%ls] file: [%ls]", *TextureName.ToString(),*ResourcePath );
		return false;
	}

	Image.ChangeFormat(ERawImageFormat::BGRA8,EGammaSpace::sRGB);

	OutWidth = Image.SizeX;
	OutHeight = Image.SizeY;
	OutDecodedImage = MoveTemp(Image.RawData);

	return true;
}

FSlateShaderResourceProxy* FSlateRHIResourceManager::GenerateTextureResource( const FNewTextureInfo& Info, const FName TextureName )
{
	FSlateShaderResourceProxy* NewProxy = NULL;
	const uint32 Width = Info.TextureData->GetWidth();
	const uint32 Height = Info.TextureData->GetHeight();

	if( Info.bShouldAtlas )
	{
		const FAtlasedTextureSlot* NewSlot = NULL;
		FSlateTextureAtlasRHI* Atlas = NULL;

		// See if any atlases can hold the texture
		for( int32 AtlasIndex = 0; AtlasIndex < PrecachedTextureAtlases.Num() && !NewSlot; ++AtlasIndex )
		{
			Atlas = PrecachedTextureAtlases[AtlasIndex];
			NewSlot = Atlas->AddTexture( Width, Height, Info.TextureData->GetRawBytes() );
		}

		if( !NewSlot )
		{
			INC_DWORD_STAT_BY(STAT_SlateNumTextureAtlases, 1);

			bool bCanUpdateAfterInitialization = GIsEditor;
			Atlas = new FSlateTextureAtlasRHI( AtlasSize, AtlasSize, ESlateTextureAtlasPaddingStyle::DilateBorder, bCanUpdateAfterInitialization);
			PrecachedTextureAtlases.Add( Atlas );
			NewSlot = PrecachedTextureAtlases.Last()->AddTexture( Width, Height, Info.TextureData->GetRawBytes() );
		}
		
		check( Atlas && NewSlot );

#if WITH_ATLAS_DEBUGGING
		AtlasDebugData.Add(NewSlot, TextureName);
#endif

		// Create a proxy to the atlased texture. The texture being used is the atlas itself with sub uvs to access the correct texture
		NewProxy = new FSlateShaderResourceProxy;
		NewProxy->Resource = Atlas->GetAtlasTexture();
		const uint32 Padding = NewSlot->Padding;
		NewProxy->StartUV = FVector2f((float)(NewSlot->X + Padding) / Atlas->GetWidth(), (float)(NewSlot->Y + Padding) / Atlas->GetHeight());
		NewProxy->SizeUV = FVector2f( (float)(NewSlot->Width-Padding*2) / Atlas->GetWidth(), (float)(NewSlot->Height-Padding*2) / Atlas->GetHeight() );
		NewProxy->ActualSize = FIntPoint( Width, Height );
	}
	else
	{
		NewProxy = new FSlateShaderResourceProxy;

		// Create a new standalone texture because we can't atlas this one
		FSlateTexture2DRHIRef* Texture = new FSlateTexture2DRHIRef( Width, Height, PF_B8G8R8A8, Info.TextureData, (Info.bSrgb ? TexCreate_SRGB : TexCreate_None) | TexCreate_ShaderResource );
		// Add it to the list of non atlased textures that we must clean up later
		NonAtlasedTextures.Add( Texture );

		INC_DWORD_STAT_BY( STAT_SlateNumNonAtlasedTextures, 1 );

		BeginInitResource( Texture );

		// The texture proxy only contains a single texture
		NewProxy->Resource = Texture;
		NewProxy->StartUV = FVector2f::ZeroVector;
		NewProxy->SizeUV = FVector2f::UnitVector;
		NewProxy->ActualSize = FIntPoint( Width, Height );
	}

	return NewProxy;
}

static void LoadUObjectForBrush( const FSlateBrush& InBrush )
{
	// Load the utexture
	FString Path = InBrush.GetResourceName().ToString();

	if (!Path.IsEmpty() && Path.StartsWith(FSlateBrush::UTextureIdentifier()))
	{
		FString NewPath = Path.RightChop(FSlateBrush::UTextureIdentifier().Len());
		UObject* TextureObject = LoadObject<UTexture2D>(NULL, *NewPath, NULL, LOAD_None, NULL);
		FSlateBrush* Brush = const_cast<FSlateBrush*>(&InBrush);

		// Set the texture object to a default texture to prevent constant loading of missing textures
		if( !TextureObject )
		{
			UE_LOGF(LogSlate, Warning, "Error loading UTexture from path: %ls not found", *Path);
			TextureObject = GEngine->DefaultTexture;
		}
		else
		{
			// We do this here because this deprecated system of loading textures will not report references and we dont want the Slate RHI resource manager to manage references
			TextureObject->AddToRoot();
		}

		
		Brush->SetResourceObject(TextureObject);

		UE_LOGF(LogSlate, Warning, "The texture:// method of loading UTextures for use in Slate is deprecated.  Please convert %ls to a Brush Asset", *Path);
	}
}

FSlateShaderResourceProxy* FSlateRHIResourceManager::GetShaderResource(const FSlateBrush& InBrush, FVector2f LocalSize, float DrawScale)
{
	SCOPE_CYCLE_COUNTER( STAT_SlateGetResourceTime );

	checkSlow( IsThreadSafeForSlateRendering() );

	UObject* ResourceObject = InBrush.GetResourceObject();
	FSlateShaderResourceProxy* Resource = nullptr;

	if (ResourceObject != nullptr && (!IsValidChecked(ResourceObject) || ResourceObject->IsUnreachable() || ResourceObject->HasAnyFlags(RF_BeginDestroyed)))
	{
		UE_LOGF(LogSlate, Warning, "Attempted to access resource for %ls which is pending kill, unreachable or pending destroy", *ResourceObject->GetName());
	}
	else
	{
		if (InBrush.GetImageType() == ESlateBrushImageType::Vector)
		{
			Resource = GetVectorResource(InBrush, LocalSize, DrawScale);
		}
		else if(!InBrush.IsDynamicallyLoaded() && !InBrush.HasUObject())
		{
			Resource = ResourceMap.FindRef(InBrush.GetResourceName());
		}
		else if(ResourceObject && ResourceObject->IsA<UMaterialInterface>())
		{
			FSlateMaterialResource* MaterialResource = GetMaterialResource(ResourceObject, &InBrush, nullptr, 0);
			Resource = MaterialResource->GetResourceProxy();
		}
		else if(InBrush.IsDynamicallyLoaded() || (InBrush.HasUObject()))
		{
			if(InBrush.HasUObject() && ResourceObject == nullptr)
			{
				// Hack for loading via the deprecated path
				LoadUObjectForBrush(InBrush);
			}

			Resource = FindOrCreateDynamicTextureResource(InBrush);
		}
	}

	return Resource;
}

FSlateShaderResource* FSlateRHIResourceManager::GetFontShaderResource( int32 InTextureAtlasIndex, FSlateShaderResource* FontTextureAtlas, const class UObject* FontMaterial )
{
	if( FontMaterial == nullptr )
	{
		return FontTextureAtlas;
	}
	else if (!FontMaterial->IsA<UMaterialInterface>())
	{
#if SLATE_CHECK_UOBJECT_RENDER_RESOURCES
		FString ErrorMsg = FString::Format(TEXT("Cannot render object '{0}' as a material because it's of type {1} instead of type MaterialInterface. Make sure all materials passed for rendering in slate are Material Interfaces. This is caused by bad user data (a UObject that is not a UMaterialInterface) passed for rendering."), { FontMaterial->GetFullName(), FontMaterial->GetClass()->GetName() });

		if (GSlateCheckUObjectRenderResourcesShouldLogFatal)
		{
			UE_LOGF(LogSlate, Fatal, "%ls", *ErrorMsg);
		}
		else
		{
			ensureMsgf(false, TEXT("%s"), *ErrorMsg);
		}
#endif
		return FontTextureAtlas;
	}
	else
	{
		return GetMaterialResource( FontMaterial, nullptr, FontTextureAtlas, InTextureAtlasIndex );
	}
}

ISlateAtlasProvider* FSlateRHIResourceManager::GetTextureAtlasProvider()
{
	return this;
}

TSharedPtr<FSlateDynamicTextureResource> FSlateRHIResourceManager::MakeDynamicTextureResource(FName ResourceName, uint32 Width, uint32 Height, const TArray< uint8 >& Bytes)
{
	// Make storage for the image
	FSlateTextureDataRef TextureStorage = MakeShareable(new FSlateTextureData(Width, Height, GPixelFormats[PF_B8G8R8A8].BlockBytes, Bytes));
	return MakeDynamicTextureResource(ResourceName, TextureStorage);
}

TSharedPtr<FSlateDynamicTextureResource> FSlateRHIResourceManager::MakeDynamicTextureResource(FName ResourceName, FSlateTextureDataRef TextureData)
{
	TSharedPtr<FSlateDynamicTextureResource> TextureResource;
	// Get a resource from the free list if possible
	if(DynamicTextureFreeList.Num() > 0)
	{
		TextureResource = DynamicTextureFreeList.Pop(EAllowShrinking::No);
	}
	else
	{
		// Free list is empty, we have to allocate a new resource
		TextureResource = MakeShareable(new FSlateDynamicTextureResource(nullptr));
	}

	TextureResource->Proxy->ActualSize = FIntPoint(TextureData->GetWidth(), TextureData->GetHeight());

	// TextureData hard-coded assumption to be BGRA8-SRGB
	check( TextureData->GetBytesPerPixel() == 4 );

	// Init render thread data
	FSlateDynamicTextureResource* InTextureResource = TextureResource.Get();
	FSlateTextureDataPtr InNewTextureData = TextureData;
	ENQUEUE_RENDER_COMMAND(InitNewSlateDynamicTextureResource)(
		[InTextureResource, InNewTextureData](FRHICommandListImmediate& RHICmdList)
		{
			if (InNewTextureData.IsValid())
			{
				// Set the texture to use as the texture we just loaded
				InTextureResource->RHIRefTexture->SetTextureData(InNewTextureData, PF_B8G8R8A8, TexCreate_SRGB);
			}

			// Initialize and link the rendering resource
			InTextureResource->RHIRefTexture->InitResource(RHICmdList);
		});

	// Map the new resource so we don't have to load again
	DynamicResourceMap.AddDynamicTextureResource( ResourceName, TextureResource.ToSharedRef() );
	INC_DWORD_STAT_BY(STAT_SlateNumDynamicTextures, 1);

	return TextureResource;
}

TSharedPtr<FSlateDynamicTextureResource> FSlateRHIResourceManager::GetDynamicTextureResourceByName( FName ResourceName )
{
	return DynamicResourceMap.GetDynamicTextureResource( ResourceName );
}

TSharedPtr<FSlateUTextureResource> FSlateRHIResourceManager::MakeDynamicUTextureResource(UTexture* InTextureObject)
{
	// Generated texture resource
	TSharedPtr<FSlateUTextureResource> TextureResource;

	// Data for a loaded disk image
	FNewTextureInfo Info;

	bool bUsingDeprecatedUTexturePath = false;

	bool bSucceeded = false;
	if( InTextureObject != NULL )
	{
		TextureResource = DynamicResourceMap.GetUTextureResource( InTextureObject );
		if( TextureResource.IsValid() )
		{
			// Bail out of the resource is already loaded
			return TextureResource;
		}

		bSucceeded = true;
	}
	
	if( bSucceeded )
	{
		// Get a resource from the free list if possible
		if (UTextureFreeList.Num() > 0)
		{
			TextureResource = UTextureFreeList.Pop(EAllowShrinking::No);
			TextureResource->UpdateTexture(InTextureObject);
		}
		else
		{
			// Free list is empty, we have to allocate a new resource
			TextureResource = MakeShareable(new FSlateUTextureResource(InTextureObject));
		}

		TextureResource->Proxy->ActualSize = FIntPoint(FMath::TruncToInt32(InTextureObject->GetSurfaceWidth()), FMath::TruncToInt32(InTextureObject->GetSurfaceHeight()));
	}
	else
	{
		// Add the null texture so we don't continuously try to load it.
		TextureResource = FSlateUTextureResource::NullResource;
	}

	DynamicResourceMap.AddUTextureResource(InTextureObject, TextureResource.ToSharedRef());

	return TextureResource;
}


FSlateShaderResourceProxy* FSlateRHIResourceManager::FindOrCreateDynamicTextureResource(const FSlateBrush& InBrush)
{
	checkSlow( IsThreadSafeForSlateRendering() );

	const FName ResourceName = InBrush.GetResourceName();
	if ( ResourceName.IsValid() && ResourceName != NAME_None )
	{
		if ( UObject* ResourceObject = InBrush.GetResourceObject() )
		{
			if ( UTexture* TextureObject = Cast<UTexture>(ResourceObject) )
			{
				TSharedPtr<FSlateUTextureResource> TextureResource = DynamicResourceMap.GetUTextureResource(TextureObject);

				if ( !TextureResource.IsValid() )
				{
					TextureResource = MakeDynamicUTextureResource(TextureObject);
					if ( TextureResource.IsValid() )
					{
						INC_DWORD_STAT_BY(STAT_SlateNumDynamicTextures, 1);
					}
				}

				if ( TextureResource.IsValid() && TextureResource->GetTextureObject() && TextureResource->GetTextureObject()->GetResource())
				{
					TextureResource->UpdateTexture(TextureObject);
					return TextureResource->Proxy;
				}
			}
			else if ( ISlateTextureAtlasInterface* AtlasedTextureObject = Cast<ISlateTextureAtlasInterface>(ResourceObject) )
			{
				const FSlateAtlasData& AtlasData = AtlasedTextureObject->GetSlateAtlasData();
				if ( AtlasData.AtlasTexture )
				{
					TSharedPtr<FSlateAtlasedTextureResource> AtlasResource = DynamicResourceMap.GetAtlasedTextureResource(AtlasData.AtlasTexture);

					if ( !AtlasResource.IsValid() )
					{
						AtlasResource = MakeShareable(new FSlateAtlasedTextureResource(AtlasData.AtlasTexture));
						DynamicResourceMap.AddAtlasedTextureResource(AtlasData.AtlasTexture, AtlasResource.ToSharedRef());
					}

					FSlateShaderResourceProxy* AtlasedProxy = AtlasResource->FindOrCreateAtlasedProxy(ResourceObject, AtlasData);

					return AtlasedProxy;
				}

				return nullptr;
			}
			else
			{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				static TSet<UObject*> FailedTextures;
				if ( !FailedTextures.Contains(ResourceObject) )
				{
					FailedTextures.Add(ResourceObject);

					if (IsValid(ResourceObject))
					{
						ensureMsgf(false, TEXT("Slate RHI Error - Invalid Texture2D, ResourceName: '%s', Path: '%s', Class: '%s'"),
							*ResourceName.ToString(),
							*ResourceObject->GetFullName(),
							*ResourceObject->GetClass()->GetName());
					}
					else
					{
						ensureMsgf(false, TEXT("Slate RHI Error - Invalid Texture2D, ResourceName: '%s'. UObject not valid."), *ResourceName.ToString());
					}
				}
				ResourceObject = GetBadResourceTexture();
#else
				return nullptr;
#endif
			}
		}
		else
		{
			TSharedPtr<FSlateDynamicTextureResource> TextureResource = DynamicResourceMap.GetDynamicTextureResource( ResourceName );

			if( !TextureResource.IsValid() )
			{
				uint32 Width; 
				uint32 Height;
				TArray<uint8> RawData;

				// Load the image from disk
				bool bSucceeded = LoadTexture(ResourceName, ResourceName.ToString(), Width, Height, RawData);
				if(bSucceeded)
				{
					TextureResource = MakeDynamicTextureResource(ResourceName, Width, Height, RawData);
				}
			}

			if(TextureResource.IsValid())
			{
				return TextureResource->Proxy;
			}
		}
	}

	// dynamic texture was not found or loaded
	return nullptr;
}

FSlateMaterialResource* FSlateRHIResourceManager::GetMaterialResource(const UObject* InMaterial, const FSlateBrush* InBrush, FSlateShaderResource* TextureMask, int32 InMaskKey )
{
	checkSlow(IsThreadSafeForSlateRendering());

	const UMaterialInterface* Material = CastChecked<UMaterialInterface>(InMaterial);

	FVector2f ImageSize = InBrush ? UE::Slate::CastToVector2f(InBrush->ImageSize) : FVector2f::ZeroVector;
	FMaterialKey Key(Material, ImageSize, InMaskKey);

	TSharedPtr<FSlateMaterialResource> MaterialResource = DynamicResourceMap.GetMaterialResource(Key);
	if (!MaterialResource.IsValid())
	{
		// Get a resource from the free list if possible
		if(MaterialResourceFreeList.Num() > 0)
		{
			MaterialResource = MaterialResourceFreeList.Pop();
			ensure(MaterialResource->GetResourceProxy() == nullptr);
			MaterialResource->UpdateMaterial( *Material, ImageSize, TextureMask );
		}
		else
		{
			MaterialResource = MakeShareable(new FSlateMaterialResource(*Material, ImageSize, TextureMask));
		}
		
		DynamicResourceMap.AddMaterialResource(Key, MaterialResource.ToSharedRef());
	}
	else
	{
		MaterialResource->UpdateMaterial( *Material, ImageSize, TextureMask );
	}

	return MaterialResource.Get();
}

FSlateShaderResourceProxy* FSlateRHIResourceManager::GetVectorResource(const FSlateBrush& Brush, FVector2f LocalSize, float DrawScale)
{
	return VectorGraphicsCache->GetShaderResource(Brush, LocalSize, DrawScale);
}

void FSlateRHIResourceManager::OnAppExit()
{
	FlushRenderingCommands();

	ReleaseResources();

	FlushRenderingCommands();

	DeleteResources();
}

bool FSlateRHIResourceManager::ContainsTexture( const FName& ResourceName ) const
{
	return ResourceMap.Contains( ResourceName );
}

void FSlateRHIResourceManager::ReleaseDynamicResource( const FSlateBrush& InBrush )
{
	checkSlow(IsThreadSafeForSlateRendering());

	// Note: Only dynamically loaded or utexture brushes can be dynamically released
	if( InBrush.HasUObject() || InBrush.IsDynamicallyLoaded() )
	{
		// Reset the rendering resource handle when our resource is being released
		InBrush.ResourceHandle = FSlateResourceHandle();

		FName ResourceName = InBrush.GetResourceName();

		UObject* ResourceObject = InBrush.GetResourceObject();

		if( ResourceObject && DynamicResourceMap.GetNumObjectResources() > 0 )
		{
			TSharedPtr<FSlateUTextureResource> TextureResource = DynamicResourceMap.GetUTextureResource(Cast<UTexture>(ResourceObject));

			if(TextureResource.IsValid())
			{
				//remove it from the accessed textures
				DynamicResourceMap.RemoveUTextureResource(TextureResource->GetTextureObject());
				TextureResource->ResetTexture();
				UTextureFreeList.Add(TextureResource);

				DEC_DWORD_STAT_BY(STAT_SlateNumDynamicTextures, 1);
			}
			else
			{
				UMaterialInterface* Material = Cast<UMaterialInterface>(ResourceObject);

				FMaterialKey Key(Material, UE::Slate::CastToVector2f(InBrush.ImageSize), 0);

				TSharedPtr<FSlateMaterialResource> MaterialResource = DynamicResourceMap.GetMaterialResource(Key);
				
				DynamicResourceMap.RemoveMaterialResource(Key);

				if (MaterialResource.IsValid())
				{
					MaterialResource->ResetMaterial();
					MaterialResourceFreeList.Add( MaterialResource );
				}
			}
		}
		else if( !ResourceObject )
		{
			TSharedPtr<FSlateDynamicTextureResource> TextureResource = DynamicResourceMap.GetDynamicTextureResource(ResourceName);

			// Only release the texture resource if it isn't shared by other handles
			if (TextureResource.IsValid() && (!TextureResource->Proxy || TextureResource->Proxy->HandleData.IsUnique() || !TextureResource->Proxy->HandleData.IsValid()))
			{
				// Release the rendering resource, its no longer being used
				BeginReleaseResource(TextureResource->RHIRefTexture);

				//remove it from the texture map
				DynamicResourceMap.RemoveDynamicTextureResource(ResourceName);

				DynamicTextureFreeList.Add( TextureResource );

				DEC_DWORD_STAT_BY(STAT_SlateNumDynamicTextures, 1);
			}
		}
	}
}

void FSlateRHIResourceManager::LoadUsedTextures()
{
	TArray< const FSlateBrush* > Resources;
	FSlateStyleRegistry::GetAllResources( Resources );
	 
	CreateTextures( Resources );
}

void FSlateRHIResourceManager::LoadStyleResources( const ISlateStyle& Style )
{
	TArray< const FSlateBrush* > Resources;
	Style.GetResources( Resources );

	CreateTextures( Resources );
}

void FSlateRHIResourceManager::UpdateTextureAtlases()
{
	for( int32 AtlasIndex = 0; AtlasIndex < PrecachedTextureAtlases.Num(); ++AtlasIndex )
	{
		PrecachedTextureAtlases[AtlasIndex]->ConditionalUpdateTexture();
	}

	VectorGraphicsCache->UpdateCache();
}


void FSlateRHIResourceManager::ConditionalFlushAtlases()
{
	VectorGraphicsCache->ConditionalFlushCache();
}

void FSlateRHIResourceManager::ReleaseResources()
{
	checkSlow( IsThreadSafeForSlateRendering() );

	for( int32 AtlasIndex = 0; AtlasIndex < PrecachedTextureAtlases.Num(); ++AtlasIndex )
	{
		PrecachedTextureAtlases[AtlasIndex]->ReleaseResources();
	}

	for( int32 ResourceIndex = 0; ResourceIndex < NonAtlasedTextures.Num(); ++ResourceIndex )
	{
		BeginReleaseResource( NonAtlasedTextures[ResourceIndex] );
	}

	DynamicResourceMap.ReleaseResources();

	if (VectorGraphicsCache != nullptr)
	{
		VectorGraphicsCache->ReleaseResources();
	}

	// Note the base class has texture proxies only which do not need to be released
}

void FSlateRHIResourceManager::DeleteBrushResourcesCommand()
{
	FlushRenderingCommands();

	FScopeLock ScopeLock(&ResourceCriticalSection);

	DeleteUObjectBrushResources();
}

void FSlateRHIResourceManager::DeleteResources()
{
	FScopeLock ScopeLock(&ResourceCriticalSection);

	for( int32 AtlasIndex = 0; AtlasIndex < PrecachedTextureAtlases.Num(); ++AtlasIndex )
	{
		delete PrecachedTextureAtlases[AtlasIndex];
	}

	for( int32 ResourceIndex = 0; ResourceIndex < NonAtlasedTextures.Num(); ++ResourceIndex )
	{
		delete NonAtlasedTextures[ResourceIndex];
	}

	SET_DWORD_STAT(STAT_SlateNumNonAtlasedTextures, 0);
	SET_DWORD_STAT(STAT_SlateNumTextureAtlases, 0);
	SET_DWORD_STAT(STAT_SlateNumDynamicTextures, 0);

	PrecachedTextureAtlases.Empty();
	NonAtlasedTextures.Empty();
	DynamicTextureFreeList.Empty();

	VectorGraphicsCache.Reset();

	// Clean up mapping to texture
	ClearTextureMap();

	DeleteUObjectBrushResources();

}

void FSlateRHIResourceManager::DeleteUObjectBrushResources()
{
	DynamicResourceMap.Empty();
	MaterialResourceFreeList.Empty();
	UTextureFreeList.Empty();
}

void FSlateRHIResourceManager::ReloadTextures()
{
	checkSlow( IsThreadSafeForSlateRendering() );

	// Release rendering resources
	ReleaseResources();

	// wait for all rendering resources to be released
	FlushRenderingCommands();

	// Delete allocated resources (cpu)
	DeleteResources();

	CreateVectorGraphicsCache();

	// Reload everything
	LoadUsedTextures();
}

UTexture* FSlateRHIResourceManager::GetBadResourceTexture()
{
	if ( BadResourceTexture == nullptr )
	{
		BadResourceTexture = FImageUtils::CreateCheckerboardTexture(FColor(255, 0, 255), FColor(255, 255, 0));
		BadResourceTexture->AddToRoot();
	}

	return BadResourceTexture;
}


int32 FSlateRHIResourceManager::GetSceneCount()
{
	checkSlow(IsInRenderingThread());
	return ActiveScenes.Num();
}

FSceneInterface* FSlateRHIResourceManager::GetSceneAt(int32 Index)
{
	checkSlow(IsInRenderingThread());
	return ActiveScenes[Index];
}

void FSlateRHIResourceManager::AddSceneAt(FSceneInterface* Scene, int32 Index)
{
	checkSlow(IsInRenderingThread());
	if (ActiveScenes.Num() <= Index)
	{
		ActiveScenes.SetNumZeroed(Index + 1);
	}
	ActiveScenes[Index] = Scene;
}

void FSlateRHIResourceManager::ClearScenes()
{
	checkSlow(IsInRenderingThread());
	ActiveScenes.Empty();
}

#undef LOCTEXT_NAMESPACE
