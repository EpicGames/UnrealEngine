// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "Textures/TextureAtlas.h"
#include "UObject/GCObject.h"
#include "UObject/ObjectKey.h"
#include "Containers/Queue.h"
#include "Rendering/ShaderResourceManager.h"
#include "Rendering/RenderingCommon.h"
#include "Rendering/DrawElements.h"
#include "Materials/MaterialInterface.h"
#include "Tickable.h"
#include "SlateElementVertexBuffer.h"

class FSlateAtlasedTextureResource;
class FSlateDynamicTextureResource;
class FSlateMaterialResource;
class FSlateUTextureResource;
class ISlateStyle;
class UTexture;
class FSceneInterface;
class FSlateVectorGraphicsCache;

/** 
 * Lookup key for materials.  Sometimes the same material is used with different masks so there must be
 * unique resource per material/mask combination
 */
struct FMaterialKey
{
	FObjectKey Material;
	const FVector2f ImageSize;
	int32 MaskKey;

	FMaterialKey(const UMaterialInterface* InMaterial, const FVector2f InImageSize, int32 InMaskKey)
		: Material(InMaterial)
		, ImageSize(InImageSize)
		, MaskKey(InMaskKey)
	{}

	friend bool operator==(const FMaterialKey& Lhs, const FMaterialKey& Rhs)
	{
		return Lhs.Material == Rhs.Material && Lhs.ImageSize == Rhs.ImageSize && Lhs.MaskKey == Rhs.MaskKey;
	}

	friend uint32 GetTypeHash(const FMaterialKey& Key)
	{
		return HashCombine(GetTypeHash(Key.Material), HashCombine(GetTypeHash(Key.ImageSize), Key.MaskKey));
	}
};

struct FDynamicResourceMap
{
public:
	FDynamicResourceMap();

	TSharedPtr<FSlateDynamicTextureResource> GetDynamicTextureResource( FName ResourceName ) const;

	TSharedPtr<FSlateUTextureResource> GetUTextureResource( UTexture* TextureObject ) const;

	TSharedPtr<FSlateAtlasedTextureResource> GetAtlasedTextureResource(UTexture* InObject) const;

	TSharedPtr<FSlateMaterialResource> GetMaterialResource( const FMaterialKey& InKey ) const;

	void AddUTextureResource( UTexture* TextureObject, TSharedRef<FSlateUTextureResource> InResource );
	void RemoveUTextureResource( UTexture* TextureObject );

	void AddDynamicTextureResource( FName ResourceName, TSharedRef<FSlateDynamicTextureResource> InResource);
	void RemoveDynamicTextureResource( FName ResourceName );

	void AddMaterialResource( const FMaterialKey& InKey, TSharedRef<FSlateMaterialResource> InResource );
	void RemoveMaterialResource( const FMaterialKey& InKey );

	void AddAtlasedTextureResource(UTexture* TextureObject, TSharedRef<FSlateAtlasedTextureResource> InResource);
	void RemoveAtlasedTextureResource(UTexture* TextureObject);

	FSlateShaderResourceProxy* FindOrCreateAtlasedProxy(UObject* InObject);

	void Empty();

	void EmptyUTextureResources();
	void EmptyMaterialResources();
	void EmptyDynamicTextureResources();

	void ReleaseResources();

	uint32 GetNumObjectResources() const { return TextureMap.Num() + MaterialMap.Num(); }

public:
	void RemoveExpiredTextureResources(TArray< TSharedPtr<FSlateUTextureResource> >& RemovedTextures);
	void RemoveExpiredMaterialResources(TArray< TSharedPtr<FSlateMaterialResource> >& RemovedMaterials);

private:
	TMap<FName, TSharedPtr<FSlateDynamicTextureResource> > NativeTextureMap;
	
	/** Map of all texture resources */
	typedef TMap<TWeakObjectPtr<UTexture>, TSharedPtr<FSlateUTextureResource> > FTextureResourceMap;
	FTextureResourceMap TextureMap;

	/** Map of all material resources */
	typedef TMap<FMaterialKey, TSharedPtr<FSlateMaterialResource> > FMaterialResourceMap;
	FMaterialResourceMap MaterialMap;

	/** Map of all object resources */
	typedef TMap<TWeakObjectPtr<UObject>, TSharedPtr<FSlateAtlasedTextureResource> > FObjectResourceMap;
	FObjectResourceMap ObjectMap;
};

/**
 * Stores a mapping of texture names to their RHI texture resource               
 */
class FSlateRHIResourceManager : public ISlateAtlasProvider, public FSlateShaderResourceManager, public FTickableGameObject
{
public:
	FSlateRHIResourceManager();
	virtual ~FSlateRHIResourceManager();

	/** ISlateAtlasProvider interface */
	virtual int32 GetNumAtlasPages() const override;
	virtual FSlateShaderResource* GetAtlasPageResource(const int32 InIndex) const override;
	virtual bool IsAtlasPageResourceAlphaOnly(const int32 InIndex) const override;
#if WITH_ATLAS_DEBUGGING
	virtual FAtlasSlotInfo GetAtlasSlotInfoAtPosition(FIntPoint InPosition, int32 AtlasIndex) const override;
#endif

	/** FTickableGameObject interface */
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	virtual bool IsTickableWhenPaused() const override { return true; }
	virtual bool IsTickableInEditor() const override { return true; }
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(FSlateRHIResourceManager, STATGROUP_Tickables); }
	virtual void Tick(float DeltaSeconds) override;

	/**
	 * Loads and creates rendering resources for all used textures.  
	 * In this implementation all textures must be known at startup time or they will not be found
	 */
	void LoadUsedTextures();

	void LoadStyleResources( const ISlateStyle& Style );

	/**
	 * Updates texture atlases if needed
	 */
	void UpdateTextureAtlases();
	void ConditionalFlushAtlases();

	/** FSlateShaderResourceManager interface */
	virtual FSlateShaderResourceProxy* GetShaderResource(const FSlateBrush& InBrush, FVector2f LocalSize, float DrawScale) override;
	virtual FSlateShaderResource* GetFontShaderResource( int32 InTextureAtlasIndex, FSlateShaderResource* FontTextureAtlas, const class UObject* FontMaterial ) override;
	virtual ISlateAtlasProvider* GetTextureAtlasProvider() override;

	/**
	 * Makes a dynamic texture resource and begins use of it
	 *
	 * @param InTextureObject	The texture object to create the resource from
	 */
	TSharedPtr<FSlateUTextureResource> MakeDynamicUTextureResource( UTexture* InTextureObject);
	
	/**
	 * Makes a dynamic texture resource and begins use of it
	 *
	 * @param ResourceName			The name identifier of the resource
	 * @param Width					The width of the resource
	 * @param Height				The height of the resource
	 * @param Bytes					The payload containing the resource
	 */
	TSharedPtr<FSlateDynamicTextureResource> MakeDynamicTextureResource( FName ResourceName, uint32 Width, uint32 Height, const TArray< uint8 >& Bytes );

	TSharedPtr<FSlateDynamicTextureResource> MakeDynamicTextureResource(FName ResourceName, FSlateTextureDataRef TextureData);

	/**
	 * Find a dynamic texture resource 
	 *
	 * @param ResourceName			The name identifier of the resource
	 */
	TSharedPtr<FSlateDynamicTextureResource> GetDynamicTextureResourceByName( FName ResourceName );

	/**
	 * Returns true if a texture resource with the passed in resource name is available 
	 */
	bool ContainsTexture( const FName& ResourceName ) const;

	/** Releases a specific dynamic resource */
	void ReleaseDynamicResource( const FSlateBrush& InBrush );

	/** 
	 * Creates a new texture from the given texture name
	 *
	 * @param TextureName	The name of the texture to load
	 */
	virtual bool LoadTexture( const FName& TextureName, const FString& ResourcePath, uint32& Width, uint32& Height, TArray<uint8>& DecodedImage );
	virtual bool LoadTexture( const FSlateBrush& InBrush, uint32& Width, uint32& Height, TArray<uint8>& DecodedImage );


	/**
	 * Releases rendering resources
	 */
	void ReleaseResources();

	/**
	 * Reloads texture resources for all used textures.  
	 *
	 * @param InExtraResources     Optional list of textures to create that aren't in the style.
	 */
	void ReloadTextures();

	int32 GetSceneCount();
	FSceneInterface* GetSceneAt(int32 Index);
	void AddSceneAt(FSceneInterface* Scene, int32 Index);
	void ClearScenes();

	FCriticalSection* GetResourceCriticalSection() { return &ResourceCriticalSection; }

private:
	void CreateVectorGraphicsCache();

	void OnPreGarbageCollect();
	void OnPostGarbageCollect();

	void TryToCleanupExpiredResources(bool bForceCleanup);
	void CleanupExpiredResources();

	/**
	 * Deletes resources created by the manager
	 */
	void DeleteResources();

	/**
	 * Deletes re-creatable brush resources - a soft reset.
	 */
	void DeleteUObjectBrushResources();

	/**
	 * Debugging command to try forcing a refresh of UObject brushes.
	 */
	void DeleteBrushResourcesCommand();

	/** 
	 * Creates textures from files on disk and atlases them if possible
	 *
	 * @param Resources	The brush resources to load images for
	 */
	void CreateTextures( const TArray< const FSlateBrush* >& Resources );

	/**
	 * Generates rendering resources for a texture
	 * 
	 * @param Info	Information on how to generate the texture
	 */
	FSlateShaderResourceProxy* GenerateTextureResource( const FNewTextureInfo& Info, const FName TextureName );
	
	/**
	 * Returns a texture rendering resource from for a dynamically loaded texture or utexture object
	 * Note: this will load the UTexture or image if needed 
	 *
	 * @param InBrush	Slate brush for the dynamic resource
	 */
	FSlateShaderResourceProxy* FindOrCreateDynamicTextureResource( const FSlateBrush& InBrush );

	/**
	 * Returns a rendering resource for a material
	 *
	 * @param InMaterial	The material object
	 */
	FSlateMaterialResource* GetMaterialResource( const UObject* InMaterial, const FSlateBrush* InBrush, FSlateShaderResource* TextureMask, int32 InMaskKey );

	/** 
	 * Returns a rendering resource for a brush that generates ector graphics (may generate it internally)
	 * 
	 * @param InBrush	The brush to with texture to get
	 * @param LocalSize	The unscaled local size of the final image
	 * @param DrawScale	Any scaling applied to the final image
	 */
	FSlateShaderResourceProxy* GetVectorResource(const FSlateBrush& Brush, FVector2f LocalSize, float DrawScale);

	/**
	 * Called when the application exists before the UObject system shuts down so we can free object resources
	 */
	void OnAppExit();

	/**
	 * Get or create the bad resource texture.
	 */
	UTexture* GetBadResourceTexture();

private:
	/**
	 * Necessary to grab before flushing the resource pool, as it may be being 
	 * accessed by multiple threads when loading.
	 */
	FCriticalSection ResourceCriticalSection;

	/** Was ResourceCriticalSection lock before GC and need to be released */
	bool bResourceCriticalSectionLockedForGC;

	/** Attempt to cleanup */
	bool bExpiredResourcesNeedCleanup;

	/** Map of all active dynamic resources being used by brushes */
	FDynamicResourceMap DynamicResourceMap;
	/** List of old utexture resources that are free to use as new resources */
	TArray< TSharedPtr<FSlateUTextureResource> > UTextureFreeList;
	/** List of old dynamic resources that are free to use as new resources */
	TArray< TSharedPtr<FSlateDynamicTextureResource> > DynamicTextureFreeList;
	/** List of old material resources that are free to use as new resources */
	TArray< TSharedPtr<FSlateMaterialResource> > MaterialResourceFreeList;
	/** Static Texture atlases which have been created */
	TArray<class FSlateTextureAtlasRHI*> PrecachedTextureAtlases;
	/** Static Textures created that are not atlased */	
	TArray<class FSlateTexture2DRHIRef*> NonAtlasedTextures;
	/** The size of each texture atlas (square) */
	uint32 AtlasSize;
	/** This max size of each texture in an atlas */
	FIntPoint MaxAltasedTextureSize;
	/** Needed for displaying an error texture when we end up with bad resources. */
	UTexture* BadResourceTexture;

	/**  */
	TArray<FSceneInterface*> ActiveScenes;

	/** Debugging Commands */
	FAutoConsoleCommand DeleteResourcesCommand;

	/** Cache for atlases generated from vector graphics */
	TUniquePtr<FSlateVectorGraphicsCache> VectorGraphicsCache;
};

