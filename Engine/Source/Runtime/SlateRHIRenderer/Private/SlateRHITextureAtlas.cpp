// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateRHITextureAtlas.h"
#include "Textures/SlateTextureData.h"
#include "Rendering/SlateRenderer.h"
#include "RenderingThread.h"
#include "RenderUtils.h"
#include "Slate/SlateTextures.h"


/* FSlateTextureAtlasRHI structors
 *****************************************************************************/

FSlateTextureAtlasRHI::FSlateTextureAtlasRHI( uint32 InWidth, uint32 InHeight, ESlateTextureAtlasPaddingStyle PaddingStyle, bool bUpdatesAfterInitialization)
	: FSlateTextureAtlas(InWidth, InHeight, GPixelFormats[PF_B8G8R8A8].BlockBytes, PaddingStyle, bUpdatesAfterInitialization)
	, AtlasTexture(new FSlateTexture2DRHIRef(InWidth, InHeight, PF_B8G8R8A8, NULL, TexCreate_SRGB, true)) 
{
}


FSlateTextureAtlasRHI::~FSlateTextureAtlasRHI( )
{
	if (AtlasTexture != nullptr)
	{
		delete AtlasTexture;
	}
}


FSlateShaderResource* FSlateTextureAtlasRHI::GetAtlasTexture() const
{
	return AtlasTexture;
}

/* FSlateTextureAtlasRHI interface
 *****************************************************************************/

void FSlateTextureAtlasRHI::ReleaseResources()
{
	bNeedsUpdate = false;
	BeginReleaseResource(AtlasTexture);
}

void FSlateTextureAtlasRHI::UpdateTexture_RenderThread(FRHICommandListImmediate& RHICmdList, FSlateTextureData* RenderThreadData)
{
	check(IsInRenderingThread());

	if (!AtlasTexture->IsInitialized())
	{
		AtlasTexture->InitResource(RHICmdList);
	}

	check(AtlasTexture->IsInitialized());
	
	const TArray<uint8>& RawBytes = RenderThreadData->GetRawBytes();
	const uint8 * RawBytesPtr = RawBytes.GetData();
	
	if ( RawBytesPtr != nullptr )
	{
		uint32 DestStride;
		uint8* TempData = (uint8*)RHICmdList.LockTexture2D(AtlasTexture->GetTypedResource(), 0, RLM_WriteOnly, /*out*/ DestStride, false);
		// check(DestStride == (RenderThreadData->GetBytesPerPixel() * RenderThreadData->GetWidth())); // Temporarily disable check

		if ( TempData != nullptr )
		{
			FMemory::Memcpy(TempData, RawBytesPtr, RenderThreadData->GetBytesPerPixel() * RenderThreadData->GetWidth() * RenderThreadData->GetHeight());

			RHICmdList.UnlockTexture2D(AtlasTexture->GetTypedResource(), 0, false);
		}
	}

	delete RenderThreadData;
}


/* FSlateTextureAtlas overrides
 *****************************************************************************/

void FSlateTextureAtlasRHI::ConditionalUpdateTexture( )
{
	checkSlow(IsThreadSafeForSlateRendering());

	if (bNeedsUpdate)
	{
		// Copy the game thread data. This is deleted on the render thread
		FSlateTextureData* RenderThreadData = new FSlateTextureData( AtlasWidth, AtlasHeight, BytesPerPixel, AtlasData ); 
		FSlateTextureAtlasRHI* Atlas = this;
		ENQUEUE_RENDER_COMMAND(SlateUpdateAtlasTextureCommand)(
			[Atlas, RenderThreadData](FRHICommandListImmediate& RHICmdList)
			{
				Atlas->UpdateTexture_RenderThread(RHICmdList, RenderThreadData);
			});

		bNeedsUpdate = false;

		if (!bUpdatesAfterInitialization)
		{
			EmptyAtlasData();
		}
	}
}
