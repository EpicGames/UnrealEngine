// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Textures/TextureAtlas.h"

class FRHICommandListImmediate;
class FSlateTexture2DRHIRef;
struct FSlateTextureData;

/**
 * Represents a texture atlas for use with RHI.
 */
class FSlateTextureAtlasRHI
	: public FSlateTextureAtlas
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InWidth
	 * @param InHeight
	 * @param PaddingStyle
	 */
	FSlateTextureAtlasRHI( uint32 InWidth, uint32 InHeight, ESlateTextureAtlasPaddingStyle PaddingStyle, bool bUpdatesAfterInitialization);

	/**
	 * Destructor.
	 */
	~FSlateTextureAtlasRHI( );

public:

	/**
	 * Gets the atlas' underlying texture resource.
	 *
	 * @return The texture resource.
	 */
	virtual FSlateShaderResource* GetAtlasTexture() const override;

	virtual void ReleaseResources() override;

	/**
	 * Updates the texture on the render thread.
	 *
	 * @param RHICmdList
	 * @param RenderThreadData
	 */
	void UpdateTexture_RenderThread(FRHICommandListImmediate& RHICmdList, FSlateTextureData* RenderThreadData);

public:

	// FSlateTextureAtlas overrides.

	virtual void ConditionalUpdateTexture( );

private:

	/** The texture rendering resource */
	FSlateTexture2DRHIRef* AtlasTexture;
};
