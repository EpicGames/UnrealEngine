// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateShaders.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "PipelineStateCache.h"
#include "Rendering/RenderingCommon.h"
#include "VT/VirtualTextureFeedbackResource.h"

/** Flag to determine if we are running with a color vision deficiency shader on */
EColorVisionDeficiency GSlateColorDeficiencyType = EColorVisionDeficiency::NormalVision;
int32 GSlateColorDeficiencySeverity = 0;
bool GSlateColorDeficiencyCorrection = false;
bool GSlateShowColorDeficiencyCorrectionWithDeficiency = false;

IMPLEMENT_TYPE_LAYOUT(FSlateElementPS);

IMPLEMENT_SHADER_TYPE(, FSlateElementVS, TEXT("/Engine/Private/SlateVertexShader.usf"), TEXT("Main"), SF_Vertex);

IMPLEMENT_SHADER_TYPE(, FSlateDebugOverdrawPS, TEXT("/Engine/Private/SlateElementPixelShader.usf"), TEXT("DebugOverdrawMain"), SF_Pixel );

IMPLEMENT_GLOBAL_SHADER(FSlateMaskingVS, "/Engine/Private/SlateMaskingShader.usf", "MainVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FSlateMaskingPS, "/Engine/Private/SlateMaskingShader.usf", "MainPS", SF_Pixel);

IMPLEMENT_SHADER_TYPE(, FSlateDebugBatchingPS, TEXT("/Engine/Private/SlateElementPixelShader.usf"), TEXT("DebugBatchingMain"), SF_Pixel );

#define IMPLEMENT_SLATE_PIXELSHADER_TYPE(ShaderType, bDrawDisabledEffect, bUseTextureAlpha, bUseTextureGrayscale, bIsVirtualTexture) \
	typedef TSlateElementPS<ESlateShader::ShaderType,bDrawDisabledEffect,bUseTextureAlpha, bUseTextureGrayscale, bIsVirtualTexture> TSlateElementPS##ShaderType##bDrawDisabledEffect##bUseTextureAlpha##bUseTextureGrayscale##bIsVirtualTexture##A; \
	IMPLEMENT_SHADER_TYPE(template<>,TSlateElementPS##ShaderType##bDrawDisabledEffect##bUseTextureAlpha##bUseTextureGrayscale##bIsVirtualTexture##A,TEXT("/Engine/Private/SlateElementPixelShader.usf"),TEXT("Main"),SF_Pixel);

/**
* All the different permutations of shaders used by slate. Uses #defines to avoid dynamic branches
*/
IMPLEMENT_SLATE_PIXELSHADER_TYPE(Default, false, true, true, false);
IMPLEMENT_SLATE_PIXELSHADER_TYPE(Default, false, true, true, true);
IMPLEMENT_SLATE_PIXELSHADER_TYPE(Border, false, true, false, false);
IMPLEMENT_SLATE_PIXELSHADER_TYPE(Default, true, true, true, false);
IMPLEMENT_SLATE_PIXELSHADER_TYPE(Default, true, true, true, true);
IMPLEMENT_SLATE_PIXELSHADER_TYPE(Border, true, true, false, false);
IMPLEMENT_SLATE_PIXELSHADER_TYPE(Default, false, false, true, false);
IMPLEMENT_SLATE_PIXELSHADER_TYPE(Default, false, false, true, true);
IMPLEMENT_SLATE_PIXELSHADER_TYPE(Border, false, false, false, false);
IMPLEMENT_SLATE_PIXELSHADER_TYPE(Default, true, false, true, false);
IMPLEMENT_SLATE_PIXELSHADER_TYPE(Default, true, false, true, true);
IMPLEMENT_SLATE_PIXELSHADER_TYPE(Border, true, false, false, false);


IMPLEMENT_SLATE_PIXELSHADER_TYPE(Default, true, true, false, false);
IMPLEMENT_SLATE_PIXELSHADER_TYPE(Default, true, true, false, true);
IMPLEMENT_SLATE_PIXELSHADER_TYPE(Default, true, false, false, false);
IMPLEMENT_SLATE_PIXELSHADER_TYPE(Default, true, false, false, true);
IMPLEMENT_SLATE_PIXELSHADER_TYPE(Default, false, true, false, false);
IMPLEMENT_SLATE_PIXELSHADER_TYPE(Default, false, true, false, true);
IMPLEMENT_SLATE_PIXELSHADER_TYPE(Default, false, false, false, false);
IMPLEMENT_SLATE_PIXELSHADER_TYPE(Default, false, false, false, true);

IMPLEMENT_SLATE_PIXELSHADER_TYPE(GrayscaleFont, false, true, false, false);
IMPLEMENT_SLATE_PIXELSHADER_TYPE(GrayscaleFont, true, true, false, false);

IMPLEMENT_SLATE_PIXELSHADER_TYPE(ColorFont, false, true, false, false);
IMPLEMENT_SLATE_PIXELSHADER_TYPE(ColorFont, true, true, false, false);

IMPLEMENT_SLATE_PIXELSHADER_TYPE(LineSegment, false, true, false, false);
IMPLEMENT_SLATE_PIXELSHADER_TYPE(LineSegment, true, true, false, false);

IMPLEMENT_SLATE_PIXELSHADER_TYPE(RoundedBox, false, true, false, false);
IMPLEMENT_SLATE_PIXELSHADER_TYPE(RoundedBox, true, true, false, false);

IMPLEMENT_SLATE_PIXELSHADER_TYPE(SdfFont, false, true, false, false);
IMPLEMENT_SLATE_PIXELSHADER_TYPE(SdfFont, true, true, false, false);

IMPLEMENT_SLATE_PIXELSHADER_TYPE(MsdfFont, false, true, false, false);
IMPLEMENT_SLATE_PIXELSHADER_TYPE(MsdfFont, true, true, false, false);

/** The Slate vertex declaration. */
TGlobalResource<FSlateVertexDeclaration> GSlateVertexDeclaration;
TGlobalResource<FSlateInstancedVertexDeclaration> GSlateInstancedVertexDeclaration;
TGlobalResource<FSlateMaskingVertexDeclaration> GSlateMaskingVertexDeclaration;


/************************************************************************/
/* FSlateVertexDeclaration                                              */
/************************************************************************/
void FSlateVertexDeclaration::InitRHI(FRHICommandListBase& RHICmdList)
{
	FVertexDeclarationElementList Elements;
	uint16 Stride = sizeof(FSlateVertex);
	Elements.Add(FVertexElement(0, STRUCT_OFFSET(FSlateVertex, TexCoords), VET_Float4, 0, Stride));
	Elements.Add(FVertexElement(0, STRUCT_OFFSET(FSlateVertex, MaterialTexCoords), VET_Float2, 1, Stride));
	Elements.Add(FVertexElement(0, STRUCT_OFFSET(FSlateVertex, Position), VET_Float2, 2, Stride));
	Elements.Add(FVertexElement(0, STRUCT_OFFSET(FSlateVertex, Color), VET_Color, 3, Stride));
	Elements.Add(FVertexElement(0, STRUCT_OFFSET(FSlateVertex, SecondaryColor), VET_Color, 4, Stride));
	Elements.Add(FVertexElement(0, STRUCT_OFFSET(FSlateVertex, PixelSize), VET_UShort2, 5, Stride));

	VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
}

void FSlateVertexDeclaration::ReleaseRHI()
{
	VertexDeclarationRHI.SafeRelease();
}


/************************************************************************/
/* FSlateInstancedVertexDeclaration                                     */
/************************************************************************/
void FSlateInstancedVertexDeclaration::InitRHI(FRHICommandListBase& RHICmdList)
{
	FVertexDeclarationElementList Elements;
	uint16 Stride = sizeof(FSlateVertex);
	Elements.Add(FVertexElement(0, STRUCT_OFFSET(FSlateVertex, TexCoords), VET_Float4, 0, Stride));
	Elements.Add(FVertexElement(0, STRUCT_OFFSET(FSlateVertex, MaterialTexCoords), VET_Float2, 1, Stride));
	Elements.Add(FVertexElement(0, STRUCT_OFFSET(FSlateVertex, Position), VET_Float2, 2, Stride));
	Elements.Add(FVertexElement(0, STRUCT_OFFSET(FSlateVertex, Color), VET_Color, 3, Stride));
	Elements.Add(FVertexElement(0, STRUCT_OFFSET(FSlateVertex, SecondaryColor), VET_Color, 4, Stride));
	Elements.Add(FVertexElement(1, 0, VET_Float4, 5, sizeof(FVector4f), true));
	
	VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
}

void FSlateElementPS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.HDR.Display.OutputDevice"));
	OutEnvironment.SetDefine(TEXT("USE_709"), CVar ? (CVar->GetValueOnGameThread() == (int32)EDisplayOutputFormat::SDR_Rec709) : 1);
}


/************************************************************************/
/* FSlateMaskingVertexDeclaration                                              */
/************************************************************************/
void FSlateMaskingVertexDeclaration::InitRHI(FRHICommandListBase& RHICmdList)
{
	FVertexDeclarationElementList Elements;
	uint16 Stride = sizeof(uint32);
	Elements.Add(FVertexElement(0, 0, VET_UByte4, 0, Stride));

	VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
}

void FSlateMaskingVertexDeclaration::ReleaseRHI()
{
	VertexDeclarationRHI.SafeRelease();
}

/** Uniform buffer layout for sampling virtual texture. */
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FSlateElementVirtualTextureParams, )
	SHADER_PARAMETER_TEXTURE(Texture2D<uint4>, PageTableTexture)
	SHADER_PARAMETER_UAV(RWStructuredBuffer<uint>, FeedbackBuffer)
	SHADER_PARAMETER(FUintVector4, PackedPageTableUniform0)
	SHADER_PARAMETER(FUintVector4, PackedPageTableUniform1)
	SHADER_PARAMETER(FUintVector4, PackedUniform)
	SHADER_PARAMETER(FUintVector4, FeedbackParams)
	SHADER_PARAMETER(uint32, LayerIndex)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FSlateElementVirtualTextureParams, "SlateElementVirtualTextureParams");

void FSlateElementPS::SetVirtualTextureParameters(FMeshDrawSingleShaderBindings& ShaderBindings, FVirtualTexture2DResource* InVirtualTexture)
{
	if (InVirtualTexture == nullptr)
	{
		return;
	}

	const uint32 LayerIndex = 0;

	IAllocatedVirtualTexture* AllocatedVT = InVirtualTexture->AcquireAllocatedVT();
	FRHIShaderResourceView* PhysicalView = AllocatedVT->GetPhysicalTextureSRV(LayerIndex, InVirtualTexture->bSRGB);

	ShaderBindings.Add(TextureParameter, PhysicalView);
	ShaderBindings.Add(TextureParameterSampler, InVirtualTexture->SamplerStateRHI);

	FUintVector4 PackedPageTableUniform[2];
	AllocatedVT->GetPackedPageTableUniform(PackedPageTableUniform);

	FUintVector4 PackedUniform;
	AllocatedVT->GetPackedUniform(&PackedUniform, LayerIndex);

	VirtualTexture::FFeedbackShaderParams VirtualTextureFeedbackShaderParams;
	VirtualTexture::GetFeedbackShaderParams(VirtualTextureFeedbackShaderParams);

	const FUintVector4 FeedbackParams{ 
		VirtualTextureFeedbackShaderParams.TileMask, 
		VirtualTextureFeedbackShaderParams.TileShift, 
		VirtualTextureFeedbackShaderParams.TileJitterOffset, 
		VirtualTextureFeedbackShaderParams.BufferSize };

	FSlateElementVirtualTextureParams VTParams;
	VTParams.PageTableTexture = AllocatedVT->GetPageTableTexture(LayerIndex);
	VTParams.FeedbackBuffer = VirtualTextureFeedbackShaderParams.BufferUAV;
	VTParams.PackedPageTableUniform0 = PackedPageTableUniform[0];
	VTParams.PackedPageTableUniform1 = PackedPageTableUniform[1];
	VTParams.PackedUniform = PackedUniform;
	VTParams.FeedbackParams = FeedbackParams;
	VTParams.LayerIndex = LayerIndex;

	TUniformBufferRef<FSlateElementVirtualTextureParams> VTParamsUB = CreateUniformBufferImmediate(VTParams, EUniformBufferUsage::UniformBuffer_SingleFrame);
	ShaderBindings.Add(VirtualTextureParams, VTParamsUB);
}