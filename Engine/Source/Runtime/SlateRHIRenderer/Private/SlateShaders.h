// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "RenderResource.h"
#include "ShaderParameters.h"
#include "GlobalRenderResources.h"
#include "GlobalShader.h"
#include "MeshDrawShaderBindings.h"
#include "Rendering/RenderingCommon.h"
#include "ShaderParameterStruct.h"
#include "TextureResource.h"

extern EColorVisionDeficiency GSlateColorDeficiencyType;
extern int32 GSlateColorDeficiencySeverity;
extern bool GSlateColorDeficiencyCorrection;
extern bool GSlateShowColorDeficiencyCorrectionWithDeficiency;

/**
 * The vertex declaration for the slate vertex shader
 */
class FSlateVertexDeclaration : public FRenderResource
{
public:
	FVertexDeclarationRHIRef VertexDeclarationRHI;

	virtual ~FSlateVertexDeclaration() = default;

	/** Initializes the vertex declaration RHI resource */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	/** Releases the vertex declaration RHI resource */
	virtual void ReleaseRHI() override;
};

/**
 * The vertex declaration for the slate instanced vertex shader
 */
class FSlateInstancedVertexDeclaration : public FSlateVertexDeclaration
{
public:
	virtual ~FSlateInstancedVertexDeclaration() = default;
	
	/** Initializes the vertex declaration RHI resource */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
};

class FSlateMaskingVertexDeclaration : public FRenderResource
{
public:
	FVertexDeclarationRHIRef VertexDeclarationRHI;

	virtual ~FSlateMaskingVertexDeclaration() = default;

	/** Initializes the vertex declaration RHI resource */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	/** Releases the vertex declaration RHI resource */
	virtual void ReleaseRHI() override;
};

class FSlateElementVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FSlateElementVS);

	FSlateElementVS() = default;

	FSlateElementVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}
};

class FSlateElementPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FSlateElementPS);

	FSlateElementPS() = default;

	FSlateElementPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		TextureParameter.Bind(Initializer.ParameterMap, TEXT("ElementTexture"));
		TextureParameterSampler.Bind(Initializer.ParameterMap, TEXT("ElementTextureSampler"));
		ShaderParams.Bind(Initializer.ParameterMap, TEXT("ShaderParams"));
		ShaderParams2.Bind(Initializer.ParameterMap, TEXT("ShaderParams2"));
		GammaAndAlphaValues.Bind(Initializer.ParameterMap,TEXT("GammaAndAlphaValues"));
		VirtualTextureParams.Bind(Initializer.ParameterMap, TEXT("SlateElementVirtualTextureParams"));
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	/**
	 * Sets the texture used by this shader 
	 *
	 * @param Texture	Texture resource to use when this pixel shader is bound
	 * @param SamplerState	Sampler state to use when sampling this texture
	 */
	void SetTexture(FMeshDrawSingleShaderBindings& ShaderBindings, FRHITexture* InTexture, const FSamplerStateRHIRef SamplerState)
	{
		ShaderBindings.AddTexture(TextureParameter, TextureParameterSampler, SamplerState, InTexture);
	}

	/**
	 * Sets the texture used by this shader in case a VirtualTexture is used
	 *
	 * @param InVirtualTexture	Virtual Texture resource to use when this pixel shader is bound
	 */
	void SetVirtualTextureParameters(FMeshDrawSingleShaderBindings& ShaderBindings, FVirtualTexture2DResource* InVirtualTexture);

	/**
	 * Sets shader params used by the shader
	 * 
	 * @param InShaderParams Shader params to use
	 */
	void SetShaderParams(FMeshDrawSingleShaderBindings& ShaderBindings, const FShaderParams& InShaderParams)
	{
		ShaderBindings.Add(ShaderParams, InShaderParams.PixelParams);
		ShaderBindings.Add(ShaderParams2, InShaderParams.PixelParams2);
	}

	/**
	 * Sets the display gamma.
	 *
	 * @param DisplayGamma The display gamma to use
	 */
	void SetDisplayGammaAndInvertAlphaAndContrast(FMeshDrawSingleShaderBindings& ShaderBindings, float InDisplayGamma, float bInvertAlpha, float InContrast)
	{
		FVector4f Values( 2.2f / InDisplayGamma, 1.0f/InDisplayGamma, bInvertAlpha, InContrast);

		ShaderBindings.Add(GammaAndAlphaValues, Values);
	}

private:
	LAYOUT_FIELD(FShaderResourceParameter, TextureParameter);
	LAYOUT_FIELD(FShaderResourceParameter, TextureParameterSampler);
	LAYOUT_FIELD(FShaderParameter, ShaderParams);
	LAYOUT_FIELD(FShaderParameter, ShaderParams2);
	LAYOUT_FIELD(FShaderParameter, GammaAndAlphaValues);
	LAYOUT_FIELD(FShaderUniformBufferParameter, VirtualTextureParams);
};

/** 
 * Pixel shader types for all elements
 */
template<ESlateShader ShaderType, bool bDrawDisabledEffect, bool bUseTextureAlpha=true, bool bUseTextureGrayscale=false, bool bIsVirtualTexture=false>
class TSlateElementPS : public FSlateElementPS
{
	DECLARE_SHADER_TYPE( TSlateElementPS, Global );
public:

	TSlateElementPS()
	{
	}

	/** Constructor.  Binds all parameters used by the shader */
	TSlateElementPS( const ShaderMetaType::CompiledShaderInitializerType& Initializer )
		: FSlateElementPS( Initializer )
	{
	}


	/**
	 * Modifies the compilation of this shader
	 */
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		// Set defines based on what this shader will be used for
		OutEnvironment.SetDefine(TEXT("SHADER_TYPE"), (uint32)ShaderType);
		OutEnvironment.SetDefine(TEXT("DRAW_DISABLED_EFFECT"), (uint32)( bDrawDisabledEffect ? 1 : 0 ));
		OutEnvironment.SetDefine(TEXT("USE_TEXTURE_ALPHA"), (uint32)( bUseTextureAlpha ? 1 : 0 ));
		OutEnvironment.SetDefine(TEXT("USE_MATERIALS"), (uint32)0);
		OutEnvironment.SetDefine(TEXT("USE_TEXTURE_GRAYSCALE"), (uint32)(bUseTextureGrayscale ? 1 : 0));
		OutEnvironment.SetDefine(TEXT("SAMPLE_VIRTUAL_TEXTURE"), (uint32)(bIsVirtualTexture ? 1 : 0));
		
		FSlateElementPS::ModifyCompilationEnvironment( Parameters, OutEnvironment );
	}
};

/** 
 * Pixel shader for debugging Slate overdraw
 */
class FSlateDebugOverdrawPS : public FSlateElementPS
{	
	DECLARE_SHADER_TYPE( FSlateDebugOverdrawPS, Global );
public:
	/** Indicates that this shader should be cached */
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{ 
		return true; 
	}

	FSlateDebugOverdrawPS()
	{
	}

	/** Constructor.  Binds all parameters used by the shader */
	FSlateDebugOverdrawPS( const ShaderMetaType::CompiledShaderInitializerType& Initializer )
		: FSlateElementPS( Initializer )
	{
	}
};

/** 
 * Pixel shader for debugging Slate overdraw
 */
class FSlateDebugBatchingPS : public FSlateElementPS
{	
	DECLARE_SHADER_TYPE(FSlateDebugBatchingPS, Global );
public:
	/** Indicates that this shader should be cached */
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{ 
		return true; 
	}

	FSlateDebugBatchingPS()
	{
	}

	/** Constructor.  Binds all parameters used by the shader */
	FSlateDebugBatchingPS( const ShaderMetaType::CompiledShaderInitializerType& Initializer )
		: FSlateElementPS( Initializer )
	{
		BatchColor.Bind(Initializer.ParameterMap, TEXT("BatchColor"));
	}

	/**
	* Sets shader params used by the shader
	*
	* @param InShaderParams Shader params to use
	*/
	void SetBatchColor(FMeshDrawSingleShaderBindings& ShaderBindings, const FLinearColor& InBatchColor)
	{
		ShaderBindings.Add(BatchColor, InBatchColor);
	}

private:
	LAYOUT_FIELD(FShaderParameter, BatchColor);
};

class FSlateMaskingVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FSlateMaskingVS);
	SHADER_USE_PARAMETER_STRUCT(FSlateMaskingVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_ARRAY(FVector4f, MaskRectPacked, [2])
	END_SHADER_PARAMETER_STRUCT()
};

class FSlateMaskingPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FSlateMaskingPS);
	SHADER_USE_PARAMETER_STRUCT(FSlateMaskingPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
	END_SHADER_PARAMETER_STRUCT()
};

/** The simple element vertex declaration. */
extern TGlobalResource<FSlateVertexDeclaration> GSlateVertexDeclaration;

/** The instanced simple element vertex declaration. */
extern TGlobalResource<FSlateInstancedVertexDeclaration> GSlateInstancedVertexDeclaration;

/** The vertex declaration for rendering stencil masks. */
extern TGlobalResource<FSlateMaskingVertexDeclaration> GSlateMaskingVertexDeclaration;
