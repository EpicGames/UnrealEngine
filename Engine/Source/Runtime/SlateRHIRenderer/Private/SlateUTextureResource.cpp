// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateUTextureResource.h"

#include "RenderUtils.h"
#include "SlateRHIRendererSettings.h"

FSlateBaseUTextureResource::FSlateBaseUTextureResource(UTexture* InTexture)
	: TextureObject(InTexture)
	, CachedSlatePostBuffers(ESlatePostRT::None)
	, CachedIsVirtualTexture(false)
{
#if SLATE_CHECK_UOBJECT_RENDER_RESOURCES
	ObjectWeakPtr = InTexture;
	UpdateDebugName();
#endif

#if WITH_EDITOR
	// Use a looser condition when in editor because the texture may still be compiling which would return false for IsCurrentlyVirtualTextured()
	CachedIsVirtualTexture = TextureObject != nullptr && TextureObject->VirtualTextureStreaming && UseVirtualTexturing(GMaxRHIShaderPlatform);
#else
	CachedIsVirtualTexture = TextureObject != nullptr && TextureObject->IsCurrentlyVirtualTextured();
#endif
}

FSlateBaseUTextureResource::~FSlateBaseUTextureResource()
{
}

uint32 FSlateBaseUTextureResource::GetWidth() const
{
	return TextureObject->GetSurfaceWidth();
}

uint32 FSlateBaseUTextureResource::GetHeight() const
{
	return TextureObject->GetSurfaceHeight();
}

ESlateShaderResource::Type FSlateBaseUTextureResource::GetType() const
{
	return ESlateShaderResource::TextureObject;
}

ESlatePostRT FSlateBaseUTextureResource::GetUsedSlatePostBuffers() const
{
	return CachedSlatePostBuffers;
}

bool FSlateBaseUTextureResource::RequiresVirtualTextureFeedback() const
{
	return CachedIsVirtualTexture;
}

#if SLATE_CHECK_UOBJECT_RENDER_RESOURCES
void FSlateBaseUTextureResource::UpdateDebugName()
{
	if (TextureObject)
	{
		DebugName = TextureObject->GetFName();
	}
	else
	{
		DebugName = NAME_None;
	}
}

void FSlateBaseUTextureResource::CheckForStaleResources() const
{
	if (DebugName != NAME_None && GSlateCheckUObjectRenderResources)
	{
		// pending kill objects may still be rendered for a frame so it is valid for the check to pass
		const bool bEvenIfPendingKill = true;
		// This test needs to be thread safe.  It doesn't give us as many chances to trap bugs here but it is still useful
		const bool bThreadSafe = true;
		if (!ObjectWeakPtr.IsValid(bEvenIfPendingKill, bThreadSafe))
		{
			if (GSlateCheckUObjectRenderResourcesShouldLogFatal)
			{
				UE_LOGF(LogSlate, Fatal, "Texture %ls has become invalid. This means the resource was garbage collected while slate was using it", *DebugName.ToString());
			}
			else
			{
				checkf(false, TEXT("Texture %s has become invalid. This means the resource was garbage collected while slate was using it"), *DebugName.ToString());
			}
		}
	}
}
#endif


TSharedPtr<FSlateUTextureResource> FSlateUTextureResource::NullResource = MakeShareable( new FSlateUTextureResource(nullptr) );

FSlateUTextureResource::FSlateUTextureResource(UTexture* InTexture)
	: FSlateBaseUTextureResource(InTexture)
	, Proxy(new FSlateShaderResourceProxy)
{
	if(TextureObject)
	{

		Proxy->ActualSize = InTexture 
			? FIntPoint(InTexture->GetSurfaceWidth(), InTexture->GetSurfaceHeight())
			: FIntPoint(1, 1);
		Proxy->Resource = this;

		CachedSlatePostBuffers = ESlatePostRT::None;
		if (const USlateRHIRendererSettings* RendererSettings = USlateRHIRendererSettings::Get())
		{
			for (const TPair<ESlatePostRT, FSlatePostSettings>& SlatePostSetting : RendererSettings->GetSlatePostSettings())
			{
				const ESlatePostRT SlatePostBitflag = SlatePostSetting.Key;
				const FSlatePostSettings& SlatePostSettingValue = SlatePostSetting.Value;

				if (SlatePostSettingValue.bEnabled && InTexture && InTexture->GetPathName() == SlatePostSettingValue.GetPathToSlatePostRT())
				{
					CachedSlatePostBuffers |= SlatePostBitflag;
				}
			}
		}
	}
}

FSlateUTextureResource::~FSlateUTextureResource()
{
	if ( Proxy )
	{
		delete Proxy;
	}
}

void FSlateUTextureResource::UpdateTexture(UTexture* InTexture)
{
	TextureObject = InTexture;

#if SLATE_CHECK_UOBJECT_RENDER_RESOURCES
	ObjectWeakPtr = TextureObject;
	UpdateDebugName();
#endif

	if (!Proxy && TextureObject)
	{
		Proxy = new FSlateShaderResourceProxy;
	}

	if (Proxy && TextureObject)
	{
		CachedSlatePostBuffers = ESlatePostRT::None;
		if (const USlateRHIRendererSettings* RendererSettings = USlateRHIRendererSettings::Get())
		{
			for (const TPair<ESlatePostRT, FSlatePostSettings>& SlatePostSetting : RendererSettings->GetSlatePostSettings())
			{
				const ESlatePostRT SlatePostBitflag = SlatePostSetting.Key;
				const FSlatePostSettings& SlatePostSettingValue = SlatePostSetting.Value;

				if (SlatePostSettingValue.bEnabled && InTexture && InTexture->GetPathName() == SlatePostSettingValue.GetPathToSlatePostRT())
				{
					CachedSlatePostBuffers |= SlatePostBitflag;
				}
			}
		}

		FTexture* TextureResource = TextureObject->GetResource();

		Proxy->Resource = this;
		// If the RHI data has changed, it's possible the underlying size of the texture has changed,
		// if that's true we need to update the actual size recorded on the proxy as well, otherwise 
		// the texture will continue to render using the wrong size.
		if (TextureResource)
		{
			Proxy->ActualSize = FIntPoint(TextureResource->GetSizeX(), TextureResource->GetSizeY());
		}
		else
		{
			Proxy->ActualSize = FIntPoint(0, 0);
		}
	}
}

void FSlateUTextureResource::ResetTexture()
{
	TextureObject = nullptr;

#if SLATE_CHECK_UOBJECT_RENDER_RESOURCES
	ObjectWeakPtr = nullptr;
	UpdateDebugName();
#endif

	CachedSlatePostBuffers = ESlatePostRT::None;

	if (Proxy)
	{
		delete Proxy;
	}
	Proxy = nullptr;
}

