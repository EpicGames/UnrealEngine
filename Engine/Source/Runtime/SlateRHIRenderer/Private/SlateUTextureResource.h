// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/App.h"
#include "Textures/SlateShaderResource.h"
#include "TextureResource.h"
#include "Engine/Texture.h"

/** A resource for rendering a UTexture object in Slate */
class FSlateBaseUTextureResource : public FSlateShaderResource
{
public:
	FSlateBaseUTextureResource(UTexture* InTexture);
	virtual ~FSlateBaseUTextureResource();

	/**
	* FSlateShaderRsourceInterface
	*/
	virtual uint32 GetWidth() const;
	virtual uint32 GetHeight() const;
	virtual ESlateShaderResource::Type GetType() const;
	virtual ESlatePostRT GetUsedSlatePostBuffers() const override;
	virtual bool RequiresVirtualTextureFeedback() const override;

	/** Gets the RHI resource used for rendering and updates the last render time for texture streaming */
	FTextureRHIRef AccessRHIResource()
	{
		if ( TextureObject && TextureObject->GetResource())
		{
			FTexture* TextureResource = TextureObject->GetResource();
			TextureResource->LastRenderTime = FApp::GetCurrentTime();

			return TextureResource->TextureRHI;
		}

		return FTextureRHIRef();
	}

	UTexture* GetTextureObject() const { return TextureObject; }


#if SLATE_CHECK_UOBJECT_RENDER_RESOURCES
	virtual void CheckForStaleResources() const override;
#endif

protected:
#if SLATE_CHECK_UOBJECT_RENDER_RESOURCES
	void UpdateDebugName();
#endif

	/** Texture UObject.  Note: lifetime is managed externally */
	UTexture* TextureObject;

#if SLATE_CHECK_UOBJECT_RENDER_RESOURCES
	// Used to guard against crashes when the material object is deleted.  This is expensive so we do not do it in shipping
	TWeakObjectPtr<UTexture> ObjectWeakPtr;
	FName DebugName;
#endif

	/** Cached slate SlatePostRT assets / buffers in use */
	ESlatePostRT CachedSlatePostBuffers;
	/** Cached result of whether we use virtual textures */
	bool CachedIsVirtualTexture;
};


/** A resource for rendering a UTexture object in Slate */
class FSlateUTextureResource : public FSlateBaseUTextureResource
{
public:
	static TSharedPtr<FSlateUTextureResource> NullResource;

	FSlateUTextureResource(UTexture* InTexture);
	virtual ~FSlateUTextureResource();

	/**
	 * Updates the rendering resource with a potentially new texture
	 */
	void UpdateTexture(UTexture* InTexture);
	void ResetTexture();
public:
	/** Slate rendering proxy */
	FSlateShaderResourceProxy* Proxy;
};
