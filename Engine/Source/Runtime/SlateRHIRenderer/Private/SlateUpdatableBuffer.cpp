// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateUpdatableBuffer.h"
#include "RenderingThread.h"

DECLARE_CYCLE_STAT(TEXT("UpdateInstanceBuffer Time"), STAT_SlateUpdateInstanceBuffer, STATGROUP_Slate);

FSlateUpdatableInstanceBuffer::FSlateUpdatableInstanceBuffer(int32 InitialInstanceCount)
{
	Proxy = new FRenderProxy;
	Proxy->InstanceBufferResource.Init(InitialInstanceCount);
}

FSlateUpdatableInstanceBuffer::~FSlateUpdatableInstanceBuffer()
{
	ENQUEUE_RENDER_COMMAND(SlateUpdatableInstanceBuffer_DeleteProxy)(
		[Ptr = Proxy](FRHICommandListImmediate&)
	{
		delete Ptr;
	});

	Proxy = nullptr;
}

void FSlateUpdatableInstanceBuffer::Update(FSlateInstanceBufferData& Data)
{
	check(IsThreadSafeForSlateRendering());

	NumInstances = Data.Num();
	if (NumInstances > 0)
	{
		// Enqueue a render thread command to update the proxy with the new data.
		ENQUEUE_RENDER_COMMAND(SlateUpdatableInstanceBuffer_Update)(
			[Ptr = Proxy, LocalDataRT = MoveTemp(Data)](FRHICommandListImmediate& RHICmdList) mutable
		{
			Ptr->Update(RHICmdList, LocalDataRT);
		});
	}
}

void FSlateUpdatableInstanceBuffer::FRenderProxy::Update(FRHICommandListImmediate& RHICmdList, FSlateInstanceBufferData& Data)
{
	SCOPE_CYCLE_COUNTER(STAT_SlateUpdateInstanceBuffer);

	InstanceBufferResource.PreFillBuffer(RHICmdList, Data.Num(), false);

	int32 RequiredVertexBufferSize = Data.Num() * Data.GetTypeSize();
	uint8* InstanceBufferData = (uint8*)RHICmdList.LockBuffer(InstanceBufferResource.VertexBufferRHI, 0, RequiredVertexBufferSize, RLM_WriteOnly);

	FMemory::Memcpy(InstanceBufferData, Data.GetData(), RequiredVertexBufferSize);
	
	RHICmdList.UnlockBuffer(InstanceBufferResource.VertexBufferRHI);
}