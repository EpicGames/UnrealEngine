// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Rendering/RenderingCommon.h"
#include "RenderingThread.h"
#include "Interfaces/ISlateRHIRendererModule.h"
#include "SlateElementVertexBuffer.h"

/**
 * Represents a per instance data buffer for a custom Slate mesh element
 */
class FSlateUpdatableInstanceBuffer final : public ISlateUpdatableInstanceBuffer
{
	// Owned by the render thread
	struct FRenderProxy final : public ISlateUpdatableInstanceBufferRenderProxy
	{
		TSlateElementVertexBuffer<FVector4> InstanceBufferResource;

		virtual ~FRenderProxy()
		{
			InstanceBufferResource.Destroy();
		}

		void Update(FRHICommandListImmediate& RHICmdList, FSlateInstanceBufferData& Data);

		FRHIBuffer* GetRHI() const override
		{
			return InstanceBufferResource.VertexBufferRHI;
		}
	} *Proxy;

public:
	FSlateUpdatableInstanceBuffer(int32 InstanceCount);
	~FSlateUpdatableInstanceBuffer();

private:
	// BEGIN ISlateUpdatableInstanceBuffer
	virtual uint32 GetNumInstances() const override final { return NumInstances; }
	virtual ISlateUpdatableInstanceBufferRenderProxy* GetRenderProxy() const override final { return Proxy; }
	virtual void Update(FSlateInstanceBufferData& Data) override final;
	// END ISlateUpdatableInstanceBuffer


private:
	uint32 NumInstances = 0;
};
