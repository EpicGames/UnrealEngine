// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FX/SlateFXBaseSubsystem.h"

#include "SlateFXSubsystem.generated.h"

#define UE_API SLATERHIRENDERER_API

class FSlateRHIPostBufferProcessorProxy;
class USlatePostBufferProcessor;

UCLASS(MinimalAPI, DisplayName = "Slate FX Subsystem (RHI)")
class USlateFXSubsystem : public USlateFXBaseSubsystem
{
	GENERATED_BODY()

public:

	static UE_API TSharedPtr<FSlateRHIPostBufferProcessorProxy> GetPostProcessorProxy(ESlatePostRT InSlatePostBufferBit);

	//~ Begin UObject Interface.
	UE_API virtual void BeginDestroy() override;
	//~ End UObject Interface.

	/** Get post processor proxy for a particular post buffer index, if it exists */
	UE_API TSharedPtr<FSlateRHIPostBufferProcessorProxy> GetSlatePostProcessorProxy(ESlatePostRT InPostBufferBit);

	/** Get post processor for a particular post buffer index, if it exists */
	UE_API USlatePostBufferProcessor* GetSlatePostProcessor(ESlatePostRT InPostBufferBit);

protected:

	virtual void OnInitProcessors() override;
	virtual void OnCleanupProcessors() override;

private:

	static TSharedPtr<FSlateRHIPostBufferProcessorProxy> CreatePostBufferProcessorProxy(TSubclassOf<USlatePostBufferProcessor> PostProcessorClass);

	/** Map of post RT buffer index to buffer processor renderthread proxies, if they exist */
	TMap<ESlatePostRT, TSharedPtr<FSlateRHIPostBufferProcessorProxy>> SlatePostBufferProcessorProxies;
};

#undef UE_API
