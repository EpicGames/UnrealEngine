// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FX/SlateRHIPostBufferProcessor.h"

#define UE_API SLATERHIRENDERER_API

/**
 * Proxy for post buffer processor that the renderthread uses to perform processing
 * This proxy exists because generally speaking usage on UObjects on the renderthread
 * is a race condition due to UObjects being managed / updated by the game thread
 */
class FSlatePostBufferBlurProxy : public FSlateRHIPostBufferProcessorProxy
{

public:

	//~ Begin FSlateRHIPostBufferProcessorProxy Interface
	UE_API virtual void PostProcess_Renderthread(FRDGBuilder& GraphBuilder, const FScreenPassTexture& InputTexture, const FScreenPassTexture& OutputTexture) override;
	UE_API virtual void OnUpdateValuesRenderThread() override;
	//~ End FSlateRHIPostBufferProcessorProxy Interface

	/** Blur strength to use when processing, renderthread version actually used to draw. Must be updated via render command except during initialization. */
	float GaussianBlurStrength_RenderThread = 10;

	/** 
	 * Blur strength can be updated from both renderthread during draw and gamethread update. 
	 * Store the last value gamethread provided so we know if we should use the renderthread value or gamethread value. 
	 * We will use the most recently updated one.
	 */
	float GaussianBlurStrengthPreDraw = 10;

protected:

	/** Fence to allow for us to queue only one update per draw command from the gamethread */
	FRenderCommandFence ParamUpdateFence;
};

#undef UE_API
