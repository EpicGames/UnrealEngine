// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TextureRenderTarget2D.h"
#include "FX/SlatePostBufferProcessor.h"
#include "Layout/PaintGeometry.h"
#include "Rendering/RenderingCommon.h"
#include "Rendering/SlateRenderer.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "Interfaces/SlateRHIRenderingPolicyInterface.h"
#include "ScreenPass.h"

class USlateRHIPostBufferProcessor;

/**
 * Proxy for post buffer processor that the renderthread uses to perform processing
 * This proxy exists because generally speaking usage on UObjects on the renderthread
 * is a race condition due to UObjects being managed / updated by the game thread
 */
class FSlateRHIPostBufferProcessorProxy : public FSlatePostBufferProcessorProxy
{

public:

	virtual ~FSlateRHIPostBufferProcessorProxy() = default;

	/** Called on the render thread to run a post processing operation on the input texture and produce the output texture. */
	virtual void PostProcess_Renderthread(FRDGBuilder& GraphBuilder, const FScreenPassTexture& InputTexture, const FScreenPassTexture& OutputTexture)
	{
	}

	/** 
	 * Called when an post buffer update element is added to a renderbatch, 
	 * gives proxies a chance to queue updates to their renderthread values based on the UObject processor.
	 * These updates should likely be guarded by an 'FRenderCommandFence' to avoid duplicate updates
	 */
	virtual void OnUpdateValuesRenderThread()
	{
	}

	/**
	 * Set the UObject that we are a renderthread proxy for, useful for doing gamethread updates from the proxy
	 */
	void SetOwningProcessorObject(USlatePostBufferProcessor* InParentObject)
	{
		ParentObject = InParentObject;
	}

protected:

	/** Pointer to processor that we are a proxy for, external design constraints should ensure that this is always valid */
	TWeakObjectPtr<USlatePostBufferProcessor> ParentObject;
};

/**
 * Base class for types that can process the backbuffer scene into the slate post buffer.
 * 
 * You need to create a renderthread proxy that derives from 'FSlateRHIPostBufferProcessorProxy'
 * For an example see: USlatePostBufferBlur.
 */
class UE_DEPRECATED(5.8, "Use USlatePostBufferProcessor instead") USlateRHIPostBufferProcessor : public UObject
{

public:

	virtual ~USlateRHIPostBufferProcessor() = default;

	/**
	 * Gets proxy for this post buffer processor, for execution on the renderthread
	 */
	virtual TSharedPtr<FSlateRHIPostBufferProcessorProxy> GetRenderThreadProxy()
	{
		return nullptr;
	}
};
