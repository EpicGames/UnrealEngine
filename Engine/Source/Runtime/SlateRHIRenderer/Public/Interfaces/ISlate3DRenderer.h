// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FRHICommandListImmediate;
class FRDGBuilder;
class FSlateDrawBuffer;
class FTextureRenderTarget2DResource;

class ISlate3DRenderer : public TSharedFromThis<ISlate3DRenderer, ESPMode::ThreadSafe>
{
public:
	virtual ~ISlate3DRenderer() = default;

public:
	/** Acquire the draw buffer and release it at the end of the scope. */
	struct FScopedAcquireDrawBuffer
	{
		FScopedAcquireDrawBuffer(ISlate3DRenderer& InSlateRenderer, bool bInDeferRenderTargetUpdate = false)
			: SlateRenderer(InSlateRenderer)
			, DrawBuffer(InSlateRenderer.AcquireDrawBuffer())
			, bDeferRenderTargetUpdate(bInDeferRenderTargetUpdate)
		{
		}
		~FScopedAcquireDrawBuffer()
		{
			if (!bDeferRenderTargetUpdate)
			{
				SlateRenderer.ReleaseDrawBuffer(DrawBuffer);
			}
		}
		FScopedAcquireDrawBuffer(const FScopedAcquireDrawBuffer&) = delete;
		FScopedAcquireDrawBuffer& operator=(const FScopedAcquireDrawBuffer&) = delete;

		FSlateDrawBuffer& GetDrawBuffer()
		{
			return DrawBuffer;
		}

	private:
		ISlate3DRenderer& SlateRenderer;
		FSlateDrawBuffer& DrawBuffer;
		bool bDeferRenderTargetUpdate;
	};

	/** set if this renderer should render in gamma space by default. */
	virtual void SetUseGammaCorrection(bool bUseGammaCorrection) = 0;
	virtual void SetApplyColorDeficiencyCorrection(bool bApplyColorCorrection) = 0;

	/** @return The free buffer for drawing */
	UE_DEPRECATED(5.1, "Use ISlate3DRenderer::AcquireDrawBuffer instead and release the draw buffer.")
	virtual FSlateDrawBuffer& GetDrawBuffer()
	{
		return AcquireDrawBuffer();
	}

	/** @return The free buffer for drawing */
	virtual FSlateDrawBuffer& AcquireDrawBuffer() = 0;

	/** Return the previously acquired buffer. */
	virtual void ReleaseDrawBuffer(FSlateDrawBuffer& InWindowDrawBuffer) = 0;

	/** 
	 * Batches the draw elements in the buffer to prepare it for rendering.
	 * Call in the game thread before sending to the render thread.
	 *
	 * @param DrawBuffer The draw buffer to prepare
	 */
	virtual void DrawWindow_GameThread(FSlateDrawBuffer& DrawBuffer) = 0;

	/** 
	 * Renders the batched draw elements of the draw buffer to the given render target.
	 * Call after preparing the draw buffer and render target on the game thread.
	 *
	 * @param RenderTarget The render target to render the contents of the draw buffer to
	 * @param InDrawBuffer The draw buffer containing the batched elements to render
	 */
	virtual void DrawWindowToTarget_RenderThread(FRDGBuilder& GraphBuilder, const struct FRenderThreadUpdateContext& Context) = 0;

	UE_DEPRECATED(5.5, "Use the RDG version instead.")
	virtual void DrawWindowToTarget_RenderThread(FRHICommandListImmediate& RHICmdList, const struct FRenderThreadUpdateContext& Context) {}
};

typedef TSharedPtr<ISlate3DRenderer, ESPMode::ThreadSafe> ISlate3DRendererPtr;
