// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Rendering/RenderingCommon.h"
#include "Modules/ModuleInterface.h"
#include "Rendering/SlateRenderer.h"
#include "ISlate3DRenderer.h"
#include "Fonts/FontTypes.h"

namespace SlateRHIConstants
{
	// Number of vertex and index buffers we swap between when drawing windows
	const int32 NumBuffers = 3;
}

/**
 * Interface for the Slate RHI Renderer module.
 */
class ISlateRHIRendererModule
	: public IModuleInterface
{
public:

	/**
	 * Creates a Slate RHI renderer.
	 *
	 * @return A new renderer.
	 */
	virtual TSharedRef<FSlateRenderer> CreateSlateRHIRenderer( ) = 0;

	virtual TSharedRef<ISlate3DRenderer, ESPMode::ThreadSafe> CreateSlate3DRenderer(bool bUseGammaCorrection) = 0;

	virtual TSharedRef<ISlateFontAtlasFactory> CreateSlateFontAtlasFactory() = 0;

	virtual TSharedRef<ISlateUpdatableInstanceBuffer> CreateInstanceBuffer( int32 InitialInstanceCount ) = 0;
};
