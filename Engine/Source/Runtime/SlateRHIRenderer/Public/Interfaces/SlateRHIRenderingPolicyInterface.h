// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"

struct FSlatePostProcessSimpleBlurPassInputs
{
	FScreenPassTexture InputTexture;
	FScreenPassTexture OutputTexture;
	float Strength = 0.0f;
};

SLATERHIRENDERER_API void AddSlatePostProcessCopy(FRDGBuilder& GraphBuilder, FScreenPassTexture Input, FScreenPassTexture Output);
SLATERHIRENDERER_API void AddSlatePostProcessBlurPass(FRDGBuilder& GraphBuilder, const FSlatePostProcessSimpleBlurPassInputs& Inputs);

//////////////////////////////////////////////////////////////////////////
// Deprecated API

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "Rendering/RenderingCommon.h"
#include "Rendering/ShaderResourceManager.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingPolicy.h"

class UE_DEPRECATED(5.5, "FSlateRHIRenderingPolicyInterface is no longer used.") FSlateRHIRenderingPolicyInterface
{
public:
	FSlateRHIRenderingPolicyInterface(const class FSlateRHIRenderingPolicy* InRenderingPolicy) {}

	static int32 GetProcessSlatePostBuffers() { return 0; }

	bool IsValid() const { return false; }
	bool IsVertexColorInLinearSpace() const { return false; }
	bool GetApplyColorDeficiencyCorrection() const { return false; }
	void BlurRectExternal(FRHICommandListImmediate& RHICmdList, FRHITexture* BlurSrc, FRHITexture* BlurDst, FIntRect SrcRect, FIntRect DstRect, float BlurStrength) const {}
};

class UE_DEPRECATED(5.5, "Use ICustomSlateElement instead.") ICustomSlateElementRHI : public ICustomSlateElement
{
public:
	UE_DEPRECATED(5.5, "Use Draw_RenderThread with an RDG builder instead")
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual void Draw_RHIRenderThread(FRHICommandListImmediate& RHICmdList, const FTextureRHIRef& RenderTarget, const FSlateCustomDrawParams& Params, FSlateRHIRenderingPolicyInterface RenderingPolicyInterface) {}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

//////////////////////////////////////////////////////////////////////////