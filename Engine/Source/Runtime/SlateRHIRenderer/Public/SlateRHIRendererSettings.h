// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Engine/DeveloperSettings.h"
#include "Misc/EnumRange.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"
#include "Rendering/SlateRendererTypes.h"
#include "Templates/SubclassOf.h"

#include "SlateRHIRendererSettings.generated.h"

#define UE_API SLATERHIRENDERER_API

class USlatePostBufferProcessor;
class UTextureRenderTarget2D;

/**
 * Resolution of a particular Slate Post RT.
 */
UENUM(BlueprintType)
enum class ESlatePostResolution : uint8
{
	/** Full screen (window) resolution */
	Full = 0,
	/** Half resolution in each dimension (quarter by area) */
	Half = 1
};

/**
 * Settings for a particular Slate Post RT.
 * Notably if enabled & blur by default. To be updated with additional effects & to be expandable in game code / settings.
 */
USTRUCT(BlueprintType, meta=(HiddenByDefault, DisableSplitPin))
struct FSlatePostSettings
{
	GENERATED_BODY()

public:

	UE_API FSlatePostSettings();

	friend class USlateRHIRendererSettings;
	friend class USlateFXSubsytem;

public:

	/** Should this post buffer be enabled for updating */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BufferSettings, meta=(PinHiddenByDefault))
	uint8 bEnabled:1;

	/** Resolution of the post buffer */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BufferSettings)
	ESlatePostResolution Resolution;

	/** Copy of actually loaded post processor class */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BufferSettings, Meta = (AllowAbstract = false))
	TSubclassOf<USlatePostBufferProcessor> PostProcessorClass;

private:

	/** Path to Slate Post RT Asset */
	UPROPERTY()
	FString PathToSlatePostRT;
	
public:

	/** Get asset name for post RT texture */
	const FString& GetPathToSlatePostRT() const { return PathToSlatePostRT; }

private:

	/** Cached load of Slate Post RT Asset */
	UPROPERTY(Transient)
	TObjectPtr<UTextureRenderTarget2D> CachedSlatePostRT;

	/** True if we attempted to load the post RT asset already */
	bool bLoadAttempted:1;
};

/**
 * Settings used to control slate rendering
 */
UCLASS(MinimalAPI, config = Game, defaultconfig)
class USlateRHIRendererSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	static const USlateRHIRendererSettings* Get()
	{
		if (!IsClassLoaded<USlateRHIRendererSettings>())
		{
			return nullptr;
		}
		return GetDefault<USlateRHIRendererSettings>();
	}

	static USlateRHIRendererSettings* GetMutable()
	{
		if (!IsClassLoaded<USlateRHIRendererSettings>())
		{
			return nullptr;
		}
		return GetMutableDefault<USlateRHIRendererSettings>();
	}

public:

	UE_API USlateRHIRendererSettings();
	UE_API ~USlateRHIRendererSettings();

	//~ Begin UObject Interface.
	UE_API virtual void BeginDestroy() override;
	//~ End UObject Interface.

public:

	/** Get settings struct for a particular post buffer index */
	UFUNCTION(BlueprintCallable, Category = "SlateFX")
	UE_API FSlatePostSettings& GetMutableSlatePostSetting(ESlatePostRT InPostBufferBit);

	/** Get settings struct for a particular post buffer index */
	UFUNCTION(BlueprintCallable, Category = "SlateFX")
	UE_API const FSlatePostSettings& GetSlatePostSetting(ESlatePostRT InPostBufferBit) const;

public:

	/** Try to get post RT asset, returns nullptr if not already loaded */
	UE_API UTextureRenderTarget2D* TryGetPostBufferRT(ESlatePostRT InPostBufferBit) const;

	/** Get post RT asset, loading if not already loaded */
	UE_API UTextureRenderTarget2D* LoadGetPostBufferRT(ESlatePostRT InPostBufferBit);

	/** Get slate post settings map, non mutable */
	UE_API const TMap<ESlatePostRT, FSlatePostSettings>& GetSlatePostSettings() const;

	/** Get Slate post buffer downscale factor based on its Resolution (ESlatePostResolution, Full -> 1, Half -> 2) */
	UE_API int32 GetSlatePostBufferDownscaleFactor(ESlatePostRT InPostBufferBit) const;

private:

	/** 
	 * Map of all slate post RT's and their settings 
	 * Note that each post RT used in a frame will result in 1 full framebuffer copy for slate to sample from.
	 * If a post RT is not used, no copy will occur & that post RT will be resized to 1x1 after 2 frames of non-use.
	 * 
	 * By default only SlatePostRT_0 is enabled. The rest must manually be enabled in settings below.
	 */

	// Map is nice since needs no editor customization. After initial run there should be no more than 5 lookups each frame.
	UPROPERTY(config, EditAnywhere, EditFixedSize, Category = "PostProcessing")
	TMap<ESlatePostRT, FSlatePostSettings> SlatePostSettings;
};

#undef UE_API
