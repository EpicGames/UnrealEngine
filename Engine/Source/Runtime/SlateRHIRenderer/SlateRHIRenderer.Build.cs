// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SlateRHIRenderer : ModuleRules
{
    public SlateRHIRenderer(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"SlateBaseRenderer"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"DeveloperSettings",
				"Slate",
				"SlateCore",
                "Engine",
                "RHI",
                "RenderCore",
				"Renderer",
                "ImageCore",
				"HeadMountedDisplay"
			}
		);
	}
}
